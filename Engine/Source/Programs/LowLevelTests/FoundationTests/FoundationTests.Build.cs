// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

public class FoundationTests : TestModuleRules
{
	static FoundationTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "Foundation";
			TestMetadata.TestShortName = "Foundation";
			TestMetadata.ReportType = "testdata";
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Linux);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Mac);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Android);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.IOS);

			string PlatformCompilationArgs;
			foreach (var Platform in UnrealTargetPlatform.GetValidPlatforms())
			{
				if (Platform == UnrealTargetPlatform.Android)
				{
					PlatformCompilationArgs = "-allmodules -architectures=arm64";
				}
				else
				{
					PlatformCompilationArgs = "-allmodules";
				}
				TestMetadata.PlatformCompilationExtraArgs.Add(Platform, PlatformCompilationArgs);
			}

			// Platform-specific tags
			TestMetadata.PlatformTags.Add(UnrealTargetPlatform.Linux, "~[.]~[Slow]");
			TestMetadata.PlatformTags.Add(UnrealTargetPlatform.Android, "~[Perf]~[Slow]~[AndroidSkip]");

			// Allow Android run for this test
			// Will remove Android from PlatformsRunUnsupported as more diverse types of tests can run on this platform
			TestMetadata.PlatformsRunUnsupported.Remove(UnrealTargetPlatform.Android);
		}
	}
	public FoundationTests(ReadOnlyTargetRules Target) : base(Target, true)
	{
		bAllowUETypesInNamespaces = true;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"Cbor",
				"CoreUObject",
				"TelemetryUtils",
				"AssetRegistry",
				"Serialization"
			});

		// ZenOplogUtils relies on BuildServerInterface which only runs on desktop platforms
		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Desktop))
		{
			PrivateDependencyModuleNames.Add("ZenOplogUtils");
		}

		PrivateIncludePathModuleNames.AddRange(new string[] { 
			"NetCore"
		});

		if (Target.bBuildWithEditorOnlyData)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"DesktopPlatform",
					"ShaderPreprocessor"
				});
		}

		if (Target.bCompileAgainstEngine)
		{
			PrivateIncludePathModuleNames.AddRange(new string[]{ "SlateCore" });
		}

		if (Target.bBuildWithEditorOnlyData &&
			(Target.Platform == UnrealTargetPlatform.Mac ||
			 Target.Platform == UnrealTargetPlatform.Win64 ||
			 Target.Platform == UnrealTargetPlatform.Linux))
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"ShaderCompilerCommon",
				});
		}
	}
}