// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class FoundationTestsTarget : TestTargetRules
{
	public FoundationTestsTarget(TargetInfo Target) : base(Target)
	{
		// Collects all tests decorated with #if WITH_LOW_LEVELTESTS from dependencies
		bWithLowLevelTestsOverride = true;
		bCompileWithPluginSupport = true;

		bCompileAgainstCoreUObject = true;

		bCompileAgainstApplicationCore = true;

		bool bIsEditorPlatform = UnrealBuildTool.Utils.GetPlatformsInClass(UnrealPlatformClass.Editor).Contains(Target.Platform);
		bBuildWithEditorOnlyData = bIsEditorPlatform
			&& (Target.Configuration == UnrealTargetConfiguration.Debug || Target.Configuration == UnrealTargetConfiguration.Development);
	}
}
