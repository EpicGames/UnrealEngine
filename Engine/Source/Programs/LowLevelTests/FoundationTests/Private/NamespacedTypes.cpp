// Copyright Epic Games, Inc. All Rights Reserved.

#include "NamespacedTypes.h"

int32 UE::Tests::UNamespacedObject::BPCallable(const FNamespacedStruct& StructParam, ENamespacedEnum EnumParam, UNamespacedObject* ObjectParam, TSubclassOf<UNamespacedObject> ClassParam)
{
    return StructParam.Value + Value;
}

int32 UNonNamespacedObject::BPCallable(const UE::Tests::FNamespacedStruct& StructParam, UE::Tests::ENamespacedEnum EnumParam, UE::Tests::UNamespacedObject* ObjectParam, TSubclassOf<UE::Tests::UNamespacedObject> ClassParam)
{
    return StructParam.Value;
}