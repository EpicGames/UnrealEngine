// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SubclassOf.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"

#include "NamespacedTypes.generated.h"

namespace UE::Tests
{

UENUM(BlueprintType)
enum class ENamespacedEnum : uint8
{
    One,
    Two,
    Three,
};

UENUM(BlueprintType)
enum ENamespacedEnum2 : uint8
{
	One,
	Two,
	Three,
};

UENUM(BlueprintType)
namespace ENamespacedEnum3
{
	enum Type : uint8
	{
		One,
		Two,
		Three,
	};
}

inline uint32 GetTypeHash(ENamespacedEnum Value)
{
    return ::GetTypeHash((uint8)Value);
}

USTRUCT(BlueprintType)
struct FNamespacedStruct
{
    GENERATED_BODY()

    UPROPERTY()
    int32 Value;

    UPROPERTY()
    ENamespacedEnum Enum;

    UPROPERTY()
    TArray<ENamespacedEnum> ArrayOfEnums;

private:
	UPROPERTY()
	int64 SomePrivateProperty;
};

inline uint32 GetTypeHash(FNamespacedStruct Value)
{
    return ::GetTypeHash(Value.Value);
}

UCLASS()
class UNamespacedObject : public UObject
{
    GENERATED_BODY()

public:
    UPROPERTY()
    int32 Value;

    UPROPERTY()
    TObjectPtr<UNamespacedObject> Object;
    UPROPERTY()
    TArray<TObjectPtr<UNamespacedObject>> ArrayOfObjects;
    UPROPERTY()
    TSet<TObjectPtr<UNamespacedObject>> SetOfObjects;
    UPROPERTY()
    TMap<TObjectPtr<UNamespacedObject>, TObjectPtr<UNamespacedObject>> MapOfObjects;

    UPROPERTY()
    TSubclassOf<UNamespacedObject> Class;
    UPROPERTY()
    TArray<TSubclassOf<UNamespacedObject>> ArrayOfClasses;
    UPROPERTY()
    TSet<TSubclassOf<UNamespacedObject>> SetOfClasses;
    UPROPERTY()
    TMap<TSubclassOf<UNamespacedObject>, TSubclassOf<UNamespacedObject>> MapOfClasses;

    UPROPERTY()
    FNamespacedStruct Struct;
    UPROPERTY()
    TArray<FNamespacedStruct> ArrayOfStructs;
    UPROPERTY()
    TSet<FNamespacedStruct> SetOfStructs;
    UPROPERTY()
    TMap<FNamespacedStruct, FNamespacedStruct> MapOfStructs;

    UPROPERTY()
    ENamespacedEnum Enum;
    UPROPERTY()
    TArray<ENamespacedEnum> ArrayOfEnums;
    UPROPERTY()
    TSet<ENamespacedEnum> SetOfEnums;
    UPROPERTY()
    TMap<ENamespacedEnum, ENamespacedEnum> MapOfEnums;

    UFUNCTION(BlueprintCallable, Category=Test)
    int32 BPCallable(const FNamespacedStruct& StructParam, ENamespacedEnum EnumParam, UNamespacedObject* ObjectParam, TSubclassOf<UNamespacedObject> ClassParam);

    UFUNCTION(BlueprintImplementableEvent, Category=Test)
    void BPImplementable(const FNamespacedStruct& StructParam, ENamespacedEnum EnumParam, UNamespacedObject* ObjectParam, TSubclassOf<UNamespacedObject> ClassParam);



	// Test default values with UFUNCTION parsing
	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestFString1(FString String = FString(TEXT(""))) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestFString2(FString String = FString("")) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestFString3(FString String = FString()) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestFString4(FString String = TEXT("")) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestFString5(FString String = "") {}

#if 0
	// These aren't allowed for callable functions.
	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestUtf8String1(FUtf8String String = FUtf8String(TEXT(""))) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestUtf8String2(FUtf8String String = FUtf8String("")) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestUtf8String3(FUtf8String String = FUtf8String()) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestUtf8String4(FUtf8String String = TEXT("")) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestUtf8String5(FUtf8String String = "") {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestAnsiString1(FAnsiString String = FAnsiString(TEXT(""))) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestAnsiString2(FAnsiString String = FAnsiString("")) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestAnsiString3(FAnsiString String = FAnsiString()) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestAnsiString4(FAnsiString String = TEXT("")) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestAnsiString5(FAnsiString String = "") {}
#endif

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestFNameString1(FName String = FName(TEXT(""))) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestFNameString2(FName String = FName("")) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestFNameString3(FName String = FName()) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestFNameString4(FName String = TEXT("")) {}

	UFUNCTION(BlueprintCallable, Category = "Test")
	void TestFNameString5(FName String = "") {}

private:
	UPROPERTY()
	int64 SomePrivateProperty;
};

}

namespace UE::DifferentNameSpaceWithSameParent
{
USTRUCT()
struct FDifferentNameSpaceStructTest
{
	GENERATED_BODY()
	UPROPERTY()
	int Bob;
};
} // namespace UE::DifferentNameSpaceWithSameParent

namespace UE::InterfaceTest
{
UINTERFACE()
class UInterfaceTest : public UInterface
{
	GENERATED_BODY()
};

class IInterfaceTest
{
	GENERATED_BODY()
};
} // namespace UE::InterfaceTest

namespace UE::InterfaceTestUsage
{
UCLASS()
class UInterfaceTestUsage : public UObject, public UE::InterfaceTest::IInterfaceTest
{
	GENERATED_BODY()
};
} // namespace UE::InterfaceTestUsage

// Class referencing types inside the namespace 
UCLASS()
class UNonNamespacedObject : public UObject
{
    GENERATED_BODY()

public:
    UPROPERTY()
    TObjectPtr<UE::Tests::UNamespacedObject> Object;

    UPROPERTY()
    UE::Tests::FNamespacedStruct Struct;

    UPROPERTY()
    TArray<UE::Tests::FNamespacedStruct> ArrayOfStructs;

    UPROPERTY()
    UE::Tests::ENamespacedEnum Enum;

    UPROPERTY()
    TArray<UE::Tests::ENamespacedEnum> ArrayOfEnums;

    UFUNCTION(BlueprintCallable, Category=Test)
    int32 BPCallable(const UE::Tests::FNamespacedStruct& StructParam, UE::Tests::ENamespacedEnum EnumParam, UE::Tests::UNamespacedObject* ObjectParam, TSubclassOf<UE::Tests::UNamespacedObject> ClassParam);

    UFUNCTION(BlueprintImplementableEvent, Category=Test)
    void BPImplementable(const UE::Tests::FNamespacedStruct& StructParam, UE::Tests::ENamespacedEnum EnumParam, UE::Tests::UNamespacedObject* ObjectParam, TSubclassOf<UE::Tests::UNamespacedObject> ClassParam);
};
