// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "TestEnum.generated.h"


// Object to be used in text fixtures for various low level tests involving UObjects. 
// Should contain various kinds of properties that can be used for various purposes depending on the test. 

UENUM()
enum class ETestEnumClassInt8 : int8
{
	A,
	B = 0x7F
};

UENUM()
enum class ETestEnumClassUInt8 : uint8
{
	A,
	B = 0xFF
};

UENUM()
enum class ETestEnumClassInt : int
{
	A,
	B = 0x7FFFFFFF
};

UENUM()
enum class ETestEnumClassUnspecified
{
	A,
	B = 0x7FFFFFFF
};

UENUM()
enum class ETestEnumClassInt64 : int64
{
	A,
	B = 0x7FFFFFFFFFFFFFFF
};

UENUM()
enum class ETestEnumClassUInt64 : uint64
{
	A,
	B = 0xFFFFFFFFFFFFFFFF
};

UENUM()
namespace ETestNamespacedEnumInt8
{
	enum Type : int8
	{
		A = 0,
		B = 0x7F
	};
}

UENUM()
namespace ETestNamespacedEnumUInt8
{
	enum Type : uint8
	{
		A = 0,
		B = 0xFF
	};
}

UENUM()
namespace ETestNamespacedEnumInt
{
	enum Type : int
	{
		A = 0,
		B = 0x7FFFFFFF
	};
}

//Underlying type must be specified
//UENUM()
//namespace ETestNamespacedEnumUnspecified
//{
//	enum Type
//	{
//		A = 0,
//		B = 0x7FFFFFFF
//	};
//}

UENUM()
namespace ETestNamespacedEnumInt64
{
	enum Type : int64
	{
		A = 0,
		B = 0x7FFFFFFFFFFFFFFF
	};
}
UENUM()
namespace ETestNamespacedEnumUInt64
{
	enum Type : uint64
	{
		A = 0,
		B = 0xFFFFFFFFFFFFFFFF
	};
}

UENUM()
enum ETestRegularInt8 : int8
{
	TestRegularInt8_A = 0,
	TestRegularInt8_B = 0x7F
};

UENUM()
enum ETestRegularUInt8 : uint8
{
	TestRegularUInt8_A = 0,
	TestRegularUInt8_B = 0xFF
};

UENUM()
enum ETestRegularEnumInt : int
{
	TestRegularEnumInt_A = 0,
	TestRegularEnumInt_B = 0x7FFFFFFF
};

//Underlying type must be specified
//UENUM()
//enum ETestRegularEnumUnspecified
//{
//	TestRegularEnumUnspecified_A = 0,
//	TestRegularEnumUnspecified_B = 0x7FFFFFFF
//};

UENUM()
enum ETestRegularEnumInt64 : int64
{
	TestRegularEnumInt64_A = 0,
	TestRegularEnumInt64_B = 0x7FFFFFFFFFFFFFFF
};

UENUM()
enum ETestRegularEnumUInt64 : uint64
{
	TestRegularEnumUInt64_A = 0,
	TestRegularEnumUInt64_B = 0xFFFFFFFFFFFFFFFF
};

UCLASS()
class UTestEnum : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	ETestEnumClassInt8 EnumClassInt8;
	UPROPERTY()
	ETestEnumClassUInt8 EnumClassUInt8;
	// Unexpected underlying enum type
	//UPROPERTY()
	//ETestEnumClassInt EnumClassInt;
	UPROPERTY()
	ETestEnumClassUnspecified EnumClassUnspecified;
	UPROPERTY()
	ETestEnumClassInt64 EnumClassInt64;
	UPROPERTY()
	ETestEnumClassUInt64 EnumClassUInt64;

	//You cannot use the raw enum name as a type for member variables.
	//UPROPERTY()
	//ETestNamespacedEnumInt8::Type NamespacedInt8;
	//UPROPERTY()
	//ETestNamespacedEnumUInt8::Type NamespacedUInt8;
	//UPROPERTY()
	//ETestNamespacedEnumInt::Type NamespacedEnumInt;
	//UPROPERTY()
	//ETestNamespacedEnumInt64::Type NamespacedEnumInt64;
	//UPROPERTY()
	//ETestNamespacedEnumUInt64::Type NamespacedEnumUInt64;
	UPROPERTY()
	TEnumAsByte<ETestNamespacedEnumInt8::Type> NamespacedInt8AsByte;
	UPROPERTY()
	TEnumAsByte<ETestNamespacedEnumUInt8::Type> NamespacedUInt8AsByte;
	UPROPERTY()
	TEnumAsByte<ETestNamespacedEnumInt::Type> NamespacedEnumIntAsByte;
	UPROPERTY()
	TEnumAsByte<ETestNamespacedEnumInt64::Type> NamespacedInt64AsByte;
	UPROPERTY()
	TEnumAsByte<ETestNamespacedEnumUInt64::Type> NamespacedEnumUInt64;

	//You cannot use the raw enum name as a type for member variables.
	//UPROPERTY()
	//ETestRegularInt8 RegularInt8;
	//UPROPERTY()
	//ETestRegularUInt8 RegularUInt8;
	//UPROPERTY()
	//ETestRegularEnumInt RegularEnumInt;
	//UPROPERTY()
	//ETestRegularEnumInt64 RegularEnumInt64;
	//UPROPERTY()
	//ETestRegularEnumUInt64 RegularEnumUInt64;
	UPROPERTY()
	TEnumAsByte<ETestRegularInt8> RegularInt8AsByte;
	UPROPERTY()
	TEnumAsByte<ETestRegularUInt8> RegularUInt8AsByte;
	UPROPERTY()
	TEnumAsByte<ETestRegularEnumInt> RegularEnumIntAsByte;
	UPROPERTY()
	TEnumAsByte<ETestRegularEnumInt64> RegularEnumInt64AsByte;
	UPROPERTY()
	TEnumAsByte<ETestRegularEnumUInt64> RegularEnumUInt64AsByte;
};
