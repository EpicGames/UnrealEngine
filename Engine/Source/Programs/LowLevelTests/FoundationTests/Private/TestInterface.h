// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "CoreTypes.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"

#include "TestInterface.generated.h"

UINTERFACE(NotBlueprintable)
class UTestInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class ITestInterface
{
	GENERATED_IINTERFACE_BODY()
public:

    UFUNCTION(BlueprintCallable, Category="Test")
    virtual FName GetNumberedName(FName BaseName, int32 InNumber) = 0;
};