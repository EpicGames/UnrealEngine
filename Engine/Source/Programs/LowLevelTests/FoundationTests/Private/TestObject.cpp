// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestObject.h"
#include "UObject/CoreNet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TestObject)

UTestInterface::UTestInterface(const FObjectInitializer& ObjectInitializer)
    : Super(ObjectInitializer)
{
}

UObject* UTestObject::GetStrongObjectReference()
{
    return StrongObjectReference;
}

int32 UTestObject::GetBPOverrideableValue_Implementation()
{
    return 17;
}

FName UTestObject::GetNumberedName(FName BaseName, int32 InNumber)
{
    return FName(BaseName, InNumber);
}

void UTestObject::OnRep_StrongObjectReference()
{
}

void UTestObject::GetLifetimeReplicatedProps(TArray< FLifetimeProperty >& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
    OutLifetimeProps.Emplace(UTestObject::StaticClass()->FindPropertyByName("StrongObjectReference")->RepIndex);
}