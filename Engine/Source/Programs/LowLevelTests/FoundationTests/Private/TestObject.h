// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "TestInterface.h"
#include "UObject/Object.h"

#include "TestObject.generated.h"


// Object to be used in text fixtures for various low level tests involving UObjects. 
// Should contain various kinds of properties that can be used for various purposes depending on the test. 

USTRUCT()
struct FTestStruct
{
    GENERATED_BODY() 
    
    UPROPERTY()
    TObjectPtr<UObject> StrongObjectReference;
    
    UPROPERTY()
    TWeakObjectPtr<UObject> WeakObjectReference;

    UPROPERTY()
    TSoftObjectPtr<UObject> SoftObjectReference;
    
    UPROPERTY()
    FSoftObjectPath SoftObjectPath;
};

UCLASS()
class UTestObject : public UObject, public ITestInterface
{
    GENERATED_BODY()

public:
    UPROPERTY(ReplicatedUsing=OnRep_StrongObjectReference)
    TObjectPtr<UObject> StrongObjectReference;

#if WITH_EDITORONLY_DATA
    UPROPERTY()
    int32 EditorOnlyProp;
#endif

    UPROPERTY()
    TWeakObjectPtr<UObject> WeakObjectReference;
    
#if WITH_VERSE_VM
    UPROPERTY()
    int32 VerseOnlyProp;
#endif

    UPROPERTY()
    TSoftObjectPtr<UObject> SoftObjectReference;
    
    UPROPERTY()
    FSoftObjectPath SoftObjectPath;
    
#if WITH_EDITORONLY_DATA 
#if WITH_VERSE_VM
    UPROPERTY()
    int32 EditorVerseOnlyProp;
#endif
#endif
    
    UPROPERTY()
    FTestStruct EmbeddedStruct;
    
    UPROPERTY()
    TArray<FTestStruct> ArrayStructs;

    UFUNCTION(BlueprintCallable, Category="Test")
    UObject* GetStrongObjectReference();

    UFUNCTION(BlueprintImplementableEvent)
    void DoSomethingInBP();

    UFUNCTION(BlueprintNativeEvent)
    int32 GetBPOverrideableValue();

    UFUNCTION(BlueprintCallable, Category="Test")
    virtual FName GetNumberedName(FName BaseName, int32 InNumber) override;

    UFUNCTION()
    void OnRep_StrongObjectReference();
};