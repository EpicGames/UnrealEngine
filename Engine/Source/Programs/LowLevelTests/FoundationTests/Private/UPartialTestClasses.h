// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/UObjectPartials.h"
#include "UObject/Class.h"
#include "UPartialTestClasses.generated.h"

UCLASS(Partial)
class UTestPartialObject : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY()
	int32 NativeValue = 100;

	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
	{
		// TODO: We want to support AddReferencedObjects on Partials
		// solution should be to add the Partials ARO directly to the schema
		// when there are Partials found that has the function
		Super::AddReferencedObjects(InThis, Collector);
	}


	UFUNCTION(BlueprintPure, Category = "PartialTest")
	int32 GetFooooo() const
	{
		return 32;
	}
};

UCLASS(Partial)
class UDerivedTestPartialObject : public UTestPartialObject
{
	GENERATED_BODY()
public:
	UPROPERTY()
	int32 DerivedValue = 200;
};

UCLASS() // On purpose skip Partial attribute
class UDerivedDerivedTestPartialObject : public UDerivedTestPartialObject
{
	GENERATED_BODY()
};

UCLASS(Blueprintable, Partial)
class UBlueprintablePartialObject : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY(BlueprintReadWrite, Category = "PartialTest")
	int32 BlueprintValue = 500;
};

UCLASS(Partial)
class UPartialTestObject : public UObject
{
	GENERATED_BODY()
};

UCLASS(Partial)
class USerializationTestObject : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY()
	int32 NativeProperty = 100;

	UPROPERTY()
	FString NativeString = TEXT("Native");
};

UCLASS(Partial)
class UDerivedSerializationTestObject : public USerializationTestObject
{
	GENERATED_BODY()
public:
	UPROPERTY()
	int32 DerivedProperty = 200;
};
