// Copyright Epic Games, Inc. All Rights Reserved.

#include "UPartialTestPartials.h"

// Implementation of FTestOutlinedFunctionPartial functions
// These are outlined (not inline) to test that UHT properly generates
// function registration for non-inline partial functions

int32 FTestOutlinedFunctionPartial::GetStoredValue() const
{
	return StoredValue;
}

void FTestOutlinedFunctionPartial::SetStoredValue(int32 NewValue)
{
	StoredValue = NewValue;
}

int32 FTestOutlinedFunctionPartial::MultiplyStoredValue(int32 Multiplier)
{
	StoredValue *= Multiplier;
	return StoredValue;
}

int32 FTestOutlinedFunctionPartial::GetOwnerNativeValue() const
{
	return GetOwner().NativeValue;
}
