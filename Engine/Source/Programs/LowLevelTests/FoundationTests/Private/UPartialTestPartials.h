// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UPartialTestClasses.h"
#include "UPartialTestPartials.generated.h"

UPARTIAL(UTestPartialObject)
struct FTestObjectPartial
{
	GENERATED_BODY()

	UPROPERTY()
	int32 PartialValue = 43;

	UPROPERTY()
	FString PartialString = TEXT("PartialDefault");

	UPROPERTY()
	TArray<int32> PartialArray;

	FTestObjectPartial()
	{
		PartialArray.Add(1);
		PartialArray.Add(2);
		PartialArray.Add(3);
	}

	void BeginDestroy()
	{
		PartialArray.Empty();
	}
};

UPARTIAL(UTestPartialObject)
struct FSecondObjectPartial
{
	GENERATED_BODY()

	UPROPERTY()
	float PartialFloat = 3.14f;

	UPROPERTY()
	bool bPartialBool = true;
};

UPARTIAL(UDerivedTestPartialObject)
struct FDerivedObjectPartial
{
	GENERATED_BODY()

	UPROPERTY()
	int32 DerivedPartialValue = 999;

	UPROPERTY(EditDefaultsOnly, Category = "VALUE")
	TSubclassOf<UObject> ComponentClass;

	UPROPERTY(EditDefaultsOnly, Category = "VALUE")
	TSoftObjectPtr<UObject> SomeTexture;
};

UENUM()
enum class ETestPartialEnum : uint8
{
	ValueA,
	ValueB,
	ValueC
};

UPARTIAL(UTestPartialObject)
struct FAdvancedContainerPartial
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<int32, FString> PartialMap;

	UPROPERTY()
	TSet<int32> PartialSet;

	UPROPERTY()
	ETestPartialEnum PartialEnum = ETestPartialEnum::ValueA;

	FAdvancedContainerPartial()
	{
		PartialMap.Add(1, TEXT("One"));
		PartialMap.Add(2, TEXT("Two"));

		PartialSet.Add(10);
		PartialSet.Add(20);
		PartialSet.Add(30);
	}
};

DECLARE_DYNAMIC_DELEGATE_OneParam(FTestPartialDelegate, int32, Value);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestPartialMulticastDelegate, int32, Value);

UPARTIAL(UTestPartialObject)
struct FAdvancedTypesPartial
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UClass> PartialClass = nullptr;

	UPROPERTY()
	uint16 PartialUInt16 = 12345;

	UPROPERTY()
	uint64 PartialUInt64 = 9876543210;

	UPROPERTY()
	FTestPartialDelegate PartialDelegate;

	UPROPERTY()
	FTestPartialMulticastDelegate PartialMulticastDelegate;
};

UPARTIAL(UTestPartialObject)
struct FTestFunctionPartial
{
	GENERATED_BODY()

	UPROPERTY()
	int32 FunctionCallCount = 0;

	UFUNCTION()
	void IncrementCounter()
	{
		FunctionCallCount++;
	}

	UFUNCTION()
	int32 GetCounter() const
	{
		return FunctionCallCount;
	}

	UFUNCTION()
	int32 AddNumbers(int32 A, int32 B)
	{
		return A + B;
	}

	UFUNCTION()
	void SetOwnerValue(int32 NewValue)
	{
		GetOwner().NativeValue = NewValue;
	}
};

UPARTIAL(UTestPartialObject)
struct FTestOutlinedFunctionPartial
{
	GENERATED_BODY()

	UPROPERTY()
	int32 StoredValue = 42;

	UFUNCTION()
	FOUNDATIONTESTS_API int32 GetStoredValue() const;

	UFUNCTION()
	FOUNDATIONTESTS_API void SetStoredValue(int32 NewValue);

	UFUNCTION()
	FOUNDATIONTESTS_API int32 MultiplyStoredValue(int32 Multiplier);

	UFUNCTION()
	FOUNDATIONTESTS_API int32 GetOwnerNativeValue() const;
};

UPARTIAL(UTestPartialObject)
struct FTestEmptyPartial
{
	GENERATED_BODY()
};

UPARTIAL(UBlueprintablePartialObject)
struct FBlueprintFunctionPartial
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category = "PartialTest")
	int32 Counter = 0;

	UFUNCTION(BlueprintCallable, Category = "PartialTest")
	void IncrementBPCounter()
	{
		Counter++;
	}

	UFUNCTION(BlueprintPure, Category = "PartialTest")
	int32 GetBPCounter() const
	{
		return Counter;
	}

	UFUNCTION(BlueprintCallable, Category = "PartialTest")
	int32 MultiplyBP(int32 A, int32 B)
	{
		return A * B;
	}

	UFUNCTION(BlueprintCallable, Category = "PartialTest")
	void SetOwnerBlueprintValue(int32 NewValue)
	{
		GetOwner().BlueprintValue = NewValue;
	}

	UFUNCTION(BlueprintPure, Category = "PartialTest")
	int32 GetOwnerBlueprintValue() const
	{
		return GetOwner().BlueprintValue;
	}
};


UPARTIAL(USerializationTestObject)
struct FSerializationTestPartial
{
	GENERATED_BODY()

	UPROPERTY()
	int32 PartialInt = 42;

	UPROPERTY()
	FString PartialString = TEXT("Partial");

	UPROPERTY()
	TArray<int32> PartialArray;

	UPROPERTY()
	TOptional<int32> PartialOptional;

	UPROPERTY()
	int32 PartialStaticArray[3];

	FSerializationTestPartial()
	{
		PartialArray = {1, 2, 3};
		PartialOptional = 999;
		PartialStaticArray[0] = 10;
		PartialStaticArray[1] = 20;
		PartialStaticArray[2] = 30;
	}
};

UPARTIAL(UDerivedSerializationTestObject)
struct FDerivedSerializationTestPartial
{
	GENERATED_BODY()

	UPROPERTY()
	float DerivedPartialFloat = 3.14f;
};
