// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "TestHarness.h"
#include "TestObject.h"
#include "UObject/Package.h"

#if WITH_LOW_LEVEL_TESTS

namespace UE
{
    TEST_CASE("CoreUObject::Serialization::FArchiveReplaceObjectRef", "[CoreUObject][Serialization]")
    {
        FName PackageName = MakeUniqueObjectName(nullptr, UPackage::StaticClass(), FName("/Memory/ArchiveReplaceObjectRefTest"));
        UPackage* Package = NewObject<UPackage>(nullptr, UPackage::StaticClass(), PackageName);
        UTestObject* Source = NewObject<UTestObject>(Package, "Source");
        UTestObject* From1 = NewObject<UTestObject>(Package, "From1");
        UTestObject* To1 = NewObject<UTestObject>(Package, "To1");
        UTestObject* From2 = NewObject<UTestObject>(Package, "From2");
        UTestObject* To2 = NewObject<UTestObject>(Package, "To2");

        TMap<UObject*, UObject*> ReplacementMap;
        ReplacementMap.Add(From1, To1);
        ReplacementMap.Add(From2, To2);

        SECTION("StrongReferences", "TObjectPtr fields should be replaced")
        {
            Source->StrongObjectReference = From1;
            Source->EmbeddedStruct.StrongObjectReference = From2; 
            FTestStruct& S = Source->ArrayStructs.AddZeroed_GetRef();
            S.StrongObjectReference = From1;
            FArchiveReplaceObjectRef Ar(Source, ReplacementMap);            
            CHECK(Source->StrongObjectReference == To1);
            CHECK(Source->EmbeddedStruct.StrongObjectReference == To2);
            REQUIRE(Source->ArrayStructs.Num() == 1);
            CHECK(Source->ArrayStructs[0].StrongObjectReference == To1);
        }
        
        SECTION("WeakReferences", "TWeakObjectPtr fields should be replaced")
        {
            Source->WeakObjectReference = From1;
            Source->EmbeddedStruct.WeakObjectReference = From2;
            FTestStruct& S = Source->ArrayStructs.AddZeroed_GetRef();
            S.WeakObjectReference = From1;
            FArchiveReplaceObjectRef Ar(Source, ReplacementMap);            
            CHECK(Source->WeakObjectReference == To1);
            CHECK(Source->EmbeddedStruct.WeakObjectReference == To2);
            REQUIRE(Source->ArrayStructs.Num() == 1);
            CHECK(Source->ArrayStructs[0].WeakObjectReference == To1);
        }
        
        SECTION("SoftReferences", "TSoftObjectPtr and FSoftObjectPath fields should be replaced")
        {
            Source->SoftObjectReference = From1;
            Source->SoftObjectPath = From2;
            Source->EmbeddedStruct.SoftObjectReference = From2;
            Source->EmbeddedStruct.SoftObjectPath = From1;
            FTestStruct& S = Source->ArrayStructs.AddZeroed_GetRef();
            S.SoftObjectReference = From1;
            S.SoftObjectPath = From2;
            FArchiveReplaceObjectRef Ar(Source, ReplacementMap);            
            CHECK(Source->SoftObjectReference.GetUniqueID() == FSoftObjectPath(To1));
            CHECK(Source->SoftObjectPath == FSoftObjectPath(To2));
            CHECK(Source->EmbeddedStruct.SoftObjectReference.GetUniqueID() == FSoftObjectPath(To2));
            CHECK(Source->EmbeddedStruct.SoftObjectPath == FSoftObjectPath(To1));
            REQUIRE(Source->ArrayStructs.Num() == 1);
            CHECK(Source->ArrayStructs[0].SoftObjectReference.GetUniqueID() == FSoftObjectPath(To1));
            CHECK(Source->ArrayStructs[0].SoftObjectPath == FSoftObjectPath(To2));
        }
    }
}

#endif