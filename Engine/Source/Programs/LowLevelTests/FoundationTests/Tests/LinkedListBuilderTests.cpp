// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/LinkedListBuilder.h"
#include "Templates/Tuple.h"
#include "TestHarness.h"

struct FLinkedListElement
{
	int Value;
	FLinkedListElement* Next;
};

TEST_CASE("Validate the TLinkedListBuilder", "[CoreUObject][TLinkedListBuilder]")
{
	using FLinkedListElementBuilder = TLinkedListBuilder<FLinkedListElement, TLinkedListBuilderNextLinkMemberVar<FLinkedListElement, &FLinkedListElement::Next>>;

	TArray<FLinkedListElement> TestArray;
	for (int Index = 0; Index < 17; Index++)
	{
		TestArray.Add(FLinkedListElement{ Index, nullptr });
	}

	auto CountAndMask = [](FLinkedListElement* ListStartPtr)
		{
			int Count = 0;
			uint32 Mask = 0;
			for (FLinkedListElement* Element = ListStartPtr; Element; Element = Element->Next)
			{
				Mask |= 1u << Element->Value;
				++Count;
			}
			return MakeTuple(Count, Mask);
		};

	FLinkedListElement* ListStartPtr = nullptr;

	{
		FLinkedListElementBuilder Builder(&ListStartPtr);
		for (FLinkedListElement& Element : TestArray)
		{
			Builder.AppendNoTerminate(Element);
		}
		Builder.NullTerminate();
		TTuple<int, uint32> Results = CountAndMask(ListStartPtr);
		REQUIRE(Results.Key == 17);
		REQUIRE(Results.Value == 0x1FFFF);
	}

	{
		FLinkedListElementBuilder Builder(&ListStartPtr);
		Builder.MoveToNext();
		REQUIRE(Builder.GetListEnd() != nullptr);
		REQUIRE(Builder.GetListEnd()->Value == 1);
		Builder.Remove(TestArray[1]);
		REQUIRE(Builder.GetListEnd() != nullptr);
		REQUIRE(Builder.GetListEnd()->Value == 2);

		TTuple<int, uint32> Results = CountAndMask(ListStartPtr);
		REQUIRE(Results.Key == 16);
		REQUIRE(Results.Value == 0x1FFFD);
	}

	{
		FLinkedListElementBuilder Builder(&ListStartPtr);
		Builder.MoveToNext();
		REQUIRE(Builder.GetListEnd() != nullptr);
		REQUIRE(Builder.GetListEnd()->Value == 2);
		Builder.RemoveAll([](FLinkedListElement* Element) { return (Element->Value & 1) == 0; });
		REQUIRE(Builder.GetListEnd() != nullptr);
		REQUIRE(Builder.GetListEnd()->Value == 3);

		TTuple<int, uint32> Results = CountAndMask(ListStartPtr);
		REQUIRE(Results.Key == 7);
		REQUIRE(Results.Value == 0xAAA8);
	}
}