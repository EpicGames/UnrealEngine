// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"
#include "UObject/UObjectThreadContext.h"
#include "TestHarness.h"

struct FTestLinkerLoad : public FLinkerLoad
{
	FTestLinkerLoad(UPackage* InParent, const FPackagePath& InPackagePath)
		: FLinkerLoad(InParent, InPackagePath, 0)
	{
		FObjectImport PackageImport;
		PackageImport.ObjectName = TEXT("/Path/To/A/Package");
		PackageImport.ClassName = NAME_Package;
		PackageIndex = ImportMap.Num();
		ImportMap.Add(PackageImport);

		FObjectImport RootObjectImport;
		RootObjectImport.ObjectName = TEXT("Object");
		RootObjectImport.OuterIndex = FPackageIndex::FromImport(PackageIndex);
		RootObjectIndex = ImportMap.Num();
		ImportMap.Add(RootObjectImport);

		FObjectImport SubObject1Import;
		SubObject1Import.ObjectName = TEXT("SubObject1");
		SubObject1Import.OuterIndex = FPackageIndex::FromImport(RootObjectIndex);
		SubObject1Index = ImportMap.Num();
		ImportMap.Add(SubObject1Import);

		FObjectImport SubObject2Import;
		SubObject2Import.ObjectName = TEXT("SubObject2");
		SubObject2Import.OuterIndex = FPackageIndex::FromImport(SubObject1Index);
		SubObject2Index = ImportMap.Num();
		ImportMap.Add(SubObject2Import);
	}

	int32 PackageIndex = -1;
	int32 RootObjectIndex = -1;
	int32 SubObject1Index = -1;
	int32 SubObject2Index = -1;
};

TEST_CASE("Validate that a full object path returns import entries", "[CoreUObject][FLinkerLoad::FindImport]")
{
	FPackagePath TestPath = FPackagePath::FromPackageNameChecked("/game/TestPackage");
	TestPath.SetHeaderExtension(EPackageExtension::Asset);
	UPackage* TestRoot = NewObject<UPackage>(nullptr, TEXT("TestPackage"));
	FTestLinkerLoad* TestLinker = new FTestLinkerLoad(TestRoot, TestPath);

	// Good cases
	const FStringView TestSingleSubobject = TEXT("/Script/SomeClass /Path/To/A/Package.Object:Subobject1");
	const FStringView TestTwoSubobjects = TEXT("/Script/SomeClass /Path/To/A/Package.Object:Subobject1.Subobject2");
	const FStringView TestNoSubobjects = TEXT("/Script/SomeClass /Path/To/A/Package.Object");
	const FStringView TestTwoSubobjectsAndNoClassPath = TEXT("/Path/To/A/Package.Object:Subobject1.Subobject2");
	const FStringView TestPackage = TEXT("/Script/SomeClass /Path/To/A/Package");

	// Suspicious cases
	const FStringView TestMissingSubobject = TEXT("/Script/SomeClass /Path/To/A/Package.Object:");
	const FStringView TestMissingSubobjectWithTrailingDot = TEXT("/Script/SomeClass /Path/To/A/Package.Object:.");
	const FStringView TestValidSubobjectWithTrailingDot = TEXT("/Script/SomeClass /Path/To/A/Package.Object:Subobject1.");
	const FStringView TestTwoValidSubobjectsWithTrailingDot = TEXT("/Script/SomeClass /Path/To/A/Package.Object:Subobject1.Subobject2.");
	const FStringView TestSingleMissingSubobject = TEXT("/Script/SomeClass /Path/To/A/Package.Object:BadSubobject1");
	const FStringView TestTwoMissingSubobjects = TEXT("/Script/SomeClass /Path/To/A/Package.Object:BadSubobject1.BadSubobject2");
	const FStringView TestOneValidOneMissingSubobject = TEXT("/Script/SomeClass /Path/To/A/Package.Object:Subobject1.BadSubobject2");
	const FStringView TestMissingRootObject = TEXT("/Script/SomeClass /Path/To/A/Package.BadObject:Subobject1.Subobject2");
	const FStringView TestMissingPackage = TEXT("/Script/SomeClass /Path/To/A/BadPackage.Object:Subobject1.Subobject2");

	SECTION("Single subobject verification")
	{
		FPackageIndex Result;
		REQUIRE(TestLinker->FindImport(TestSingleSubobject, Result));
		REQUIRE(Result.IsImport());
		REQUIRE(Result.ToImport() == TestLinker->SubObject1Index);
	}

	SECTION("Two subobjects verification")
	{
		FPackageIndex Result;
		REQUIRE(TestLinker->FindImport(TestTwoSubobjects, Result));
		REQUIRE(Result.IsImport());
		REQUIRE(Result.ToImport() == TestLinker->SubObject2Index);
	}

	SECTION("No subobjects verification")
	{
		FPackageIndex Result;
		REQUIRE(TestLinker->FindImport(TestNoSubobjects, Result));
		REQUIRE(Result.IsImport());
		REQUIRE(Result.ToImport() == TestLinker->RootObjectIndex);
	}

	SECTION("No class path verification")
	{
		FPackageIndex Result;
		REQUIRE(TestLinker->FindImport(TestTwoSubobjectsAndNoClassPath, Result));
		REQUIRE(Result.IsImport());
		REQUIRE(Result.ToImport() == TestLinker->SubObject2Index);
	}

	SECTION("Package verification")
	{
		FPackageIndex Result;
		REQUIRE(TestLinker->FindImport(TestPackage, Result));
		REQUIRE(Result.IsImport());
		REQUIRE(Result.ToImport() == TestLinker->PackageIndex);
	}

	SECTION("Trailing colon still finds root object")
	{
		FPackageIndex Result;
		REQUIRE(TestLinker->FindImport(TestMissingSubobject, Result));
		REQUIRE(Result.IsImport());
		REQUIRE(Result.ToImport() == TestLinker->RootObjectIndex);
	}

	SECTION("Trailing colon and dot still finds root object")
	{
		FPackageIndex Result;
		REQUIRE(TestLinker->FindImport(TestMissingSubobjectWithTrailingDot, Result));
		REQUIRE(Result.IsImport());
		REQUIRE(Result.ToImport() == TestLinker->RootObjectIndex);
	}

	SECTION("Trailing dot still finds subobject")
	{
		FPackageIndex Result;
		REQUIRE(TestLinker->FindImport(TestValidSubobjectWithTrailingDot, Result));
		REQUIRE(Result.IsImport());
		REQUIRE(Result.ToImport() == TestLinker->SubObject1Index);
	}

	SECTION("Trailing dot still finds two subobjects")
	{
		FPackageIndex Result;
		REQUIRE(TestLinker->FindImport(TestTwoValidSubobjectsWithTrailingDot, Result));
		REQUIRE(Result.IsImport());
		REQUIRE(Result.ToImport() == TestLinker->SubObject2Index);
	}

	SECTION("Single missing subobject not found")
	{
		FPackageIndex Result;
		REQUIRE_FALSE(TestLinker->FindImport(TestSingleMissingSubobject, Result));
		REQUIRE(Result.IsNull());
	}

	SECTION("Two missing subobjects not found")
	{
		FPackageIndex Result;
		REQUIRE_FALSE(TestLinker->FindImport(TestTwoMissingSubobjects, Result));
		REQUIRE(Result.IsNull());
	}

	SECTION("One valid and one missing subobject not found")
	{
		FPackageIndex Result;
		REQUIRE_FALSE(TestLinker->FindImport(TestOneValidOneMissingSubobject, Result));
		REQUIRE(Result.IsNull());
	}

	SECTION("Missing root object not found")
	{
		FPackageIndex Result;
		REQUIRE_FALSE(TestLinker->FindImport(TestMissingRootObject, Result));
		REQUIRE(Result.IsNull());
	}

	SECTION("Missing package not found")
	{
		FPackageIndex Result;
		REQUIRE_FALSE(TestLinker->FindImport(TestMissingRootObject, Result));
		REQUIRE(Result.IsNull());
	}

	FUObjectThreadContext::Get().IsDeletingLinkers = true;
	delete TestLinker;
	FUObjectThreadContext::Get().IsDeletingLinkers = false;
}
