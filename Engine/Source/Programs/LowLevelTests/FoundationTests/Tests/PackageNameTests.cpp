// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#include "Misc/PackageName.h"
#include "TestHarness.h"

TEST_CASE("Split full object path with subobjects", "[CoreUObject][FPackageName::SplitFullObjectPath]")
{
	const FStringView ExpectedClassPath = TEXT("/Script/SomePackage.SomeClass");
	const FStringView ExpectedPackagePath = TEXT("/Path/To/A/Package");
	const FStringView ExpectedObjectName = TEXT("Object");
	const FStringView ExpectedSubobject1Name = TEXT("Subobject1");
	const FStringView ExpectedSubobject2Name = TEXT("Subobject2");

	// Good cases
	const FStringView TestSingleSubobject = TEXT("/Script/SomePackage.SomeClass /Path/To/A/Package.Object:Subobject1");
	const FStringView TestTwoSubobjects = TEXT("/Script/SomePackage.SomeClass /Path/To/A/Package.Object:Subobject1.Subobject2");
	const FStringView TestNoSubobjects = TEXT("/Script/SomePackage.SomeClass /Path/To/A/Package.Object");
	const FStringView TestTwoSubobjectsAndNoClassPath = TEXT("/Path/To/A/Package.Object:Subobject1.Subobject2");
	const FStringView TestPackage = TEXT("/Script/SomePackage.SomeClass /Path/To/A/Package");

	// Suspicious cases
	const FStringView TestMissingSubobject = TEXT("/Script/SomePackage.SomeClass /Path/To/A/Package.Object:");
	const FStringView TestMissingSubobjectWithTrailingDot = TEXT("/Script/SomePackage.SomeClass /Path/To/A/Package.Object:.");
	const FStringView TestValidSubobjectWithTrailingDot = TEXT("/Script/SomePackage.SomeClass /Path/To/A/Package.Object:Subobject1.");
	const FStringView TestTwoValidSubobjectsWithTrailingDot = TEXT("/Script/SomePackage.SomeClass /Path/To/A/Package.Object:Subobject1.Subobject2.");

	FStringView ClassPath;
	FStringView PackagePath;
	FStringView ObjectName; 
	TArray<FStringView> SubobjectNames;

	SECTION("Single subobject verification")
	{
		FPackageName::SplitFullObjectPath(TestSingleSubobject, ClassPath, PackagePath, ObjectName, SubobjectNames);

		REQUIRE(ClassPath == ExpectedClassPath);
		REQUIRE(PackagePath == ExpectedPackagePath);
		REQUIRE(ObjectName == ExpectedObjectName);
		REQUIRE(SubobjectNames.Num() == 1);
		REQUIRE(SubobjectNames[0] == ExpectedSubobject1Name);
	}

	SECTION("Two subobjects verification")
	{
		FPackageName::SplitFullObjectPath(TestTwoSubobjects, ClassPath, PackagePath, ObjectName, SubobjectNames);

		REQUIRE(ClassPath == ExpectedClassPath);
		REQUIRE(PackagePath == ExpectedPackagePath);
		REQUIRE(ObjectName == ExpectedObjectName);
		REQUIRE(SubobjectNames.Num() == 2);
		REQUIRE(SubobjectNames[0] == ExpectedSubobject1Name);
		REQUIRE(SubobjectNames[1] == ExpectedSubobject2Name);
	}

	SECTION("No subobjects verification")
	{
		FPackageName::SplitFullObjectPath(TestNoSubobjects, ClassPath, PackagePath, ObjectName, SubobjectNames);

		REQUIRE(ClassPath == ExpectedClassPath);
		REQUIRE(PackagePath == ExpectedPackagePath);
		REQUIRE(ObjectName == ExpectedObjectName);
		REQUIRE(SubobjectNames.Num() == 0);
	}

	SECTION("No class path verification (bDetectClassName on)")
	{
		FPackageName::SplitFullObjectPath(TestTwoSubobjectsAndNoClassPath, ClassPath, PackagePath, ObjectName, SubobjectNames);

		REQUIRE(ClassPath.Len() == 0);
		REQUIRE(PackagePath == ExpectedPackagePath);
		REQUIRE(ObjectName == ExpectedObjectName);
		REQUIRE(SubobjectNames.Num() == 2);
		REQUIRE(SubobjectNames[0] == ExpectedSubobject1Name);
		REQUIRE(SubobjectNames[1] == ExpectedSubobject2Name);
	}

	SECTION("No class path verification (bDetectClassName off)")
	{
		FPackageName::SplitFullObjectPath(TestTwoSubobjectsAndNoClassPath, ClassPath, PackagePath, ObjectName, SubobjectNames, false);

		REQUIRE(ClassPath.Len() == 0);
		REQUIRE(PackagePath == ExpectedPackagePath);
		REQUIRE(ObjectName == ExpectedObjectName);
		REQUIRE(SubobjectNames.Num() == 2);
		REQUIRE(SubobjectNames[0] == ExpectedSubobject1Name);
		REQUIRE(SubobjectNames[1] == ExpectedSubobject2Name);
	}

	SECTION("Package verification")
	{
		FPackageName::SplitFullObjectPath(TestPackage, ClassPath, PackagePath, ObjectName, SubobjectNames);

		REQUIRE(ClassPath == ExpectedClassPath);
		REQUIRE(PackagePath == ExpectedPackagePath);
		REQUIRE(ObjectName == TEXT(""));
		REQUIRE(SubobjectNames.Num() == 0);
	}

	SECTION("Missing subobject name yields empty subobjects array")
	{
		FPackageName::SplitFullObjectPath(TestMissingSubobject, ClassPath, PackagePath, ObjectName, SubobjectNames);

		REQUIRE(ClassPath == ExpectedClassPath);
		REQUIRE(PackagePath == ExpectedPackagePath);
		REQUIRE(ObjectName == ExpectedObjectName);
		REQUIRE(SubobjectNames.Num() == 0);
	}

	SECTION("Missing subobject name with trailing dot yields empty subobjects array")
	{
		FPackageName::SplitFullObjectPath(TestMissingSubobjectWithTrailingDot, ClassPath, PackagePath, ObjectName, SubobjectNames);

		REQUIRE(ClassPath == ExpectedClassPath);
		REQUIRE(PackagePath == ExpectedPackagePath);
		REQUIRE(ObjectName == ExpectedObjectName);
		REQUIRE(SubobjectNames.Num() == 0);
	}

	SECTION("Valid subobject with trailing dot still reports correct subobject name")
	{
		FPackageName::SplitFullObjectPath(TestValidSubobjectWithTrailingDot, ClassPath, PackagePath, ObjectName, SubobjectNames);

		REQUIRE(ClassPath == ExpectedClassPath);
		REQUIRE(PackagePath == ExpectedPackagePath);
		REQUIRE(ObjectName == ExpectedObjectName);
		REQUIRE(SubobjectNames.Num() == 1);
		REQUIRE(SubobjectNames[0] == ExpectedSubobject1Name);
	}

	SECTION("Two valid subobjects with trailing dot still reports correct subobject names")
	{
		FPackageName::SplitFullObjectPath(TestTwoValidSubobjectsWithTrailingDot, ClassPath, PackagePath, ObjectName, SubobjectNames);

		REQUIRE(ClassPath == ExpectedClassPath);
		REQUIRE(PackagePath == ExpectedPackagePath);
		REQUIRE(ObjectName == ExpectedObjectName);
		REQUIRE(SubobjectNames.Num() == 2);
		REQUIRE(SubobjectNames[0] == ExpectedSubobject1Name);
		REQUIRE(SubobjectNames[1] == ExpectedSubobject2Name);
	}
}

TEST_CASE("Split PackageName", "[CoreUObject][FPackageName::SplitPackageName]")
{
	FString PackageName(TEXT("/root/path1/path2/leaf.umap"));

	// Test that not passing outputs does not crash
	FPackageName::SplitPackageName(PackageName, nullptr, nullptr, nullptr);

	FStringView Root;
	FStringView Path;
	FStringView Leaf;
	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf);
	REQUIRE(Root == TEXTVIEW("/root/"));
	REQUIRE(Path == TEXTVIEW("path1/path2/"));
	REQUIRE(Leaf == TEXTVIEW("leaf.umap"));

	PackageName = TEXTVIEW("/root/path1/path2/");
	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf);
	REQUIRE(Root == TEXTVIEW("/root/"));
	REQUIRE(Path == TEXTVIEW("path1/path2/"));
	REQUIRE(Leaf == TEXTVIEW(""));

	PackageName = TEXTVIEW("/root/path1/leaf");
	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf);
	REQUIRE(Root == TEXTVIEW("/root/"));
	REQUIRE(Path == TEXTVIEW("path1/"));
	REQUIRE(Leaf == TEXTVIEW("leaf"));

	PackageName = TEXTVIEW("/root/path1/");
	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf);
	REQUIRE(Root == TEXTVIEW("/root/"));
	REQUIRE(Path == TEXTVIEW("path1/"));
	REQUIRE(Leaf == TEXTVIEW(""));

	PackageName = TEXTVIEW("/root/leaf");
	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf);
	REQUIRE(Root == TEXTVIEW("/root/"));
	REQUIRE(Path == TEXTVIEW(""));
	REQUIRE(Leaf == TEXTVIEW("leaf"));

	PackageName = TEXTVIEW("/root/");
	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf);
	REQUIRE(Root == TEXTVIEW("/root/"));
	REQUIRE(Path == TEXTVIEW(""));
	REQUIRE(Leaf == TEXTVIEW(""));

	PackageName = TEXTVIEW("/root");
	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf);
	REQUIRE(Root == TEXTVIEW(""));
	REQUIRE(Path == TEXTVIEW(""));
	REQUIRE(Leaf == TEXTVIEW(""));

	PackageName = TEXTVIEW("/");
	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf);
	REQUIRE(Root == TEXTVIEW(""));
	REQUIRE(Path == TEXTVIEW(""));
	REQUIRE(Leaf == TEXTVIEW(""));

	PackageName = TEXTVIEW("");
	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf);
	REQUIRE(Root == TEXTVIEW(""));
	REQUIRE(Path == TEXTVIEW(""));
	REQUIRE(Leaf == TEXTVIEW(""));

	PackageName = TEXTVIEW("path1/path2");
	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf);
	REQUIRE(Root == TEXTVIEW(""));
	REQUIRE(Path == TEXTVIEW(""));
	REQUIRE(Leaf == TEXTVIEW(""));

	// Edge cases with no contract yet; they should not crash
	PackageName = TEXTVIEW("//");
	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf);

	PackageName = TEXTVIEW("///");
	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf);

	// Path formats
	PackageName = TEXTVIEW("/root/path/leaf");
	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf, FPackageName::EPathFormatFlags::MountPointNoSlashes);
	REQUIRE(Root == TEXTVIEW("root"));
	REQUIRE(Path == TEXTVIEW("path/"));
	REQUIRE(Leaf == TEXTVIEW("leaf"));

	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf, FPackageName::EPathFormatFlags::MountPointLeadingSlash);
	REQUIRE(Root == TEXTVIEW("/root"));
	REQUIRE(Path == TEXTVIEW("path/"));
	REQUIRE(Leaf == TEXTVIEW("leaf"));

	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf, FPackageName::EPathFormatFlags::MountPointTrailingSlash);
	REQUIRE(Root == TEXTVIEW("root/"));
	REQUIRE(Path == TEXTVIEW("path/"));
	REQUIRE(Leaf == TEXTVIEW("leaf"));

	FPackageName::SplitPackageName(PackageName, &Root, &Path, &Leaf, FPackageName::EPathFormatFlags::MountPointSlashes);
	REQUIRE(Root == TEXTVIEW("/root/"));
	REQUIRE(Path == TEXTVIEW("path/"));
	REQUIRE(Leaf == TEXTVIEW("leaf"));

	// Verify Path formats are respected in the FString version
	FString RootStr;
	FString PathStr;
	FString LeafStr;
	PackageName = TEXTVIEW("/root/path/leaf");
	FPackageName::SplitPackageName(PackageName, &RootStr, &PathStr, &LeafStr, FPackageName::EPathFormatFlags::MountPointNoSlashes);
	REQUIRE(RootStr == TEXTVIEW("root"));
	REQUIRE(PathStr == TEXTVIEW("path/"));
	REQUIRE(LeafStr == TEXTVIEW("leaf"));

	FPackageName::SplitPackageName(PackageName, &RootStr, &PathStr, &LeafStr, FPackageName::EPathFormatFlags::MountPointLeadingSlash);
	REQUIRE(RootStr == TEXTVIEW("/root"));
	REQUIRE(PathStr == TEXTVIEW("path/"));
	REQUIRE(LeafStr == TEXTVIEW("leaf"));

	FPackageName::SplitPackageName(PackageName, &RootStr, &PathStr, &LeafStr, FPackageName::EPathFormatFlags::MountPointTrailingSlash);
	REQUIRE(RootStr == TEXTVIEW("root/"));
	REQUIRE(PathStr == TEXTVIEW("path/"));
	REQUIRE(LeafStr == TEXTVIEW("leaf"));

	FPackageName::SplitPackageName(PackageName, &RootStr, &PathStr, &LeafStr, FPackageName::EPathFormatFlags::MountPointSlashes);
	REQUIRE(RootStr == TEXTVIEW("/root/"));
	REQUIRE(PathStr == TEXTVIEW("path/"));
	REQUIRE(LeafStr == TEXTVIEW("leaf"));
}

TEST_CASE("Split PackageNameRoot", "[CoreUObject][FPackageName::SplitPackageNameRoot]")
{
	FString PackageName(TEXT("/root/path1/path2/leaf.umap"));

	// Test that not passing outputs does not crash
	(void) FPackageName::SplitPackageNameRoot(PackageName, nullptr);

	FStringView Root;
	FStringView RelPath;
	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath);
	REQUIRE(Root == TEXTVIEW("root"));
	REQUIRE(RelPath == TEXTVIEW("path1/path2/leaf.umap"));

	PackageName = TEXTVIEW("/root/path1/leaf");
	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath);
	REQUIRE(Root == TEXTVIEW("root"));
	REQUIRE(RelPath == TEXTVIEW("path1/leaf"));

	PackageName = TEXTVIEW("/root/");
	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath);
	REQUIRE(Root == TEXTVIEW("root"));
	REQUIRE(RelPath == TEXTVIEW(""));

	PackageName = TEXTVIEW("/root");
	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath);
	REQUIRE(Root == TEXTVIEW("root"));
	REQUIRE(RelPath == TEXTVIEW(""));

	PackageName = TEXTVIEW("/");
	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath);
	REQUIRE(Root == TEXTVIEW(""));
	REQUIRE(RelPath == TEXTVIEW(""));

	PackageName = TEXTVIEW("");
	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath);
	REQUIRE(Root == TEXTVIEW(""));
	REQUIRE(RelPath == TEXTVIEW(""));

	PackageName = TEXTVIEW("path1/path2");
	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath);
	REQUIRE(Root == TEXTVIEW(""));
	REQUIRE(RelPath == TEXTVIEW("path1/path2"));

	// Edge cases with no contract yet; they should not crash
	PackageName = TEXTVIEW("//");
	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath);

	PackageName = TEXTVIEW("///");
	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath);

	// Path formats
	PackageName = TEXTVIEW("/root/path/leaf");
	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath, FPackageName::EPathFormatFlags::MountPointNoSlashes);
	REQUIRE(Root == TEXTVIEW("root"));
	REQUIRE(RelPath == TEXTVIEW("path/leaf"));

	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath, FPackageName::EPathFormatFlags::MountPointLeadingSlash);
	REQUIRE(Root == TEXTVIEW("/root"));
	REQUIRE(RelPath == TEXTVIEW("path/leaf"));

	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath, FPackageName::EPathFormatFlags::MountPointTrailingSlash);
	REQUIRE(Root == TEXTVIEW("root/"));
	REQUIRE(RelPath == TEXTVIEW("path/leaf"));

	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath, FPackageName::EPathFormatFlags::MountPointSlashes);
	REQUIRE(Root == TEXTVIEW("/root/"));
	REQUIRE(RelPath == TEXTVIEW("path/leaf"));

	// Path formats in edge case with no second slash 
	PackageName = TEXTVIEW("/root");
	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath, FPackageName::EPathFormatFlags::MountPointNoSlashes);
	REQUIRE(Root == TEXTVIEW("root"));
	REQUIRE(RelPath == TEXTVIEW(""));

	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath, FPackageName::EPathFormatFlags::MountPointLeadingSlash);
	REQUIRE(Root == TEXTVIEW("/root"));
	REQUIRE(RelPath == TEXTVIEW(""));

	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath, FPackageName::EPathFormatFlags::MountPointTrailingSlash);
	REQUIRE(Root == TEXTVIEW("root"));
	REQUIRE(RelPath == TEXTVIEW(""));

	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath, FPackageName::EPathFormatFlags::MountPointSlashes);
	REQUIRE(Root == TEXTVIEW("/root"));
	REQUIRE(RelPath == TEXTVIEW(""));

	// Path formats in edge case with a string equal to single slash
	PackageName = TEXTVIEW("/");
	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath, FPackageName::EPathFormatFlags::MountPointNoSlashes);
	REQUIRE(Root == TEXTVIEW(""));
	REQUIRE(RelPath == TEXTVIEW(""));

	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath, FPackageName::EPathFormatFlags::MountPointLeadingSlash);
	REQUIRE(Root == TEXTVIEW("/"));
	REQUIRE(RelPath == TEXTVIEW(""));

	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath, FPackageName::EPathFormatFlags::MountPointTrailingSlash);
	REQUIRE(Root == TEXTVIEW(""));
	REQUIRE(RelPath == TEXTVIEW(""));

	Root = FPackageName::SplitPackageNameRoot(PackageName, &RelPath, FPackageName::EPathFormatFlags::MountPointSlashes);
	REQUIRE(Root == TEXTVIEW("/"));
	REQUIRE(RelPath == TEXTVIEW(""));

	// Verify Path formats are respected in the FName version
	FString RootStr;
	FString RelPathStr;
	PackageName = TEXTVIEW("/root/path/leaf");
	RootStr = FPackageName::SplitPackageNameRoot(FName(*PackageName), &RelPathStr, FPackageName::EPathFormatFlags::MountPointNoSlashes);
	REQUIRE(RootStr == TEXTVIEW("root"));
	REQUIRE(RelPathStr == TEXTVIEW("path/leaf"));

	RootStr = FPackageName::SplitPackageNameRoot(FName(*PackageName), &RelPathStr, FPackageName::EPathFormatFlags::MountPointLeadingSlash);
	REQUIRE(RootStr == TEXTVIEW("/root"));
	REQUIRE(RelPathStr == TEXTVIEW("path/leaf"));

	RootStr = FPackageName::SplitPackageNameRoot(FName(*PackageName), &RelPathStr, FPackageName::EPathFormatFlags::MountPointTrailingSlash);
	REQUIRE(RootStr == TEXTVIEW("root/"));
	REQUIRE(RelPathStr == TEXTVIEW("path/leaf"));

	RootStr = FPackageName::SplitPackageNameRoot(FName(*PackageName), &RelPathStr, FPackageName::EPathFormatFlags::MountPointSlashes);
	REQUIRE(RootStr == TEXTVIEW("/root/"));
	REQUIRE(RelPathStr == TEXTVIEW("path/leaf"));
}