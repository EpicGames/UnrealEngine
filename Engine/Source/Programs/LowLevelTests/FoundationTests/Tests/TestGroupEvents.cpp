// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestCommon/Initialization.h"

#if WITH_ENGINE
#include "Logging/LogScopedVerbosityOverride.h"
#include "SlateGlobals.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#endif

#include <catch2/catch_test_macros.hpp>

GROUP_BEFORE_GLOBAL(Catch::DefaultGroup)
{
#if WITH_ENGINE
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogSlate, ELogVerbosity::Error);
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogSlateStyle, ELogVerbosity::Error);
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogUObjectGlobals, ELogVerbosity::Fatal);
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogStreaming, ELogVerbosity::Error);
#endif

	InitAll(true, true);
}

GROUP_AFTER_GLOBAL(Catch::DefaultGroup)
{
	CleanupAll();
}
