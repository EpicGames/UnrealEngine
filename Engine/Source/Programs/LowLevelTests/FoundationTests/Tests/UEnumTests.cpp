// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#include "TestCommon/Expectations.h"
#include "TestHarness.h"
#include "TestEnum.h"
#include "UObject/Class.h"
#include "UObject/Package.h"


TEST_CASE("CoreUObject::UEnum::Basic", "[CoreUObject]")
{
	FName PackageName = MakeUniqueObjectName(nullptr, UPackage::StaticClass(), FName("/Memory/UEnumBasic"));
	UPackage* Package = NewObject<UPackage>(nullptr, UPackage::StaticClass(), PackageName);
	UTestEnum* Obj = NewObject<UTestEnum>(Package, "TestEnum");

	FEnumProperty* EnumClassInt8 = CastField<FEnumProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, EnumClassInt8)));
	CHECK_MESSAGE("EnumClassInt8", EnumClassInt8 != nullptr && EnumClassInt8->GetUnderlyingProperty()->IsA(FInt8Property::StaticClass()));
	FEnumProperty* EnumClassUInt8 = CastField<FEnumProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, EnumClassUInt8)));
	CHECK_MESSAGE("EnumClassUInt8", EnumClassUInt8 != nullptr && EnumClassUInt8->GetUnderlyingProperty()->IsA(FByteProperty::StaticClass()));
	FEnumProperty* EnumClassUnspecified = CastField<FEnumProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, EnumClassUnspecified)));
	CHECK_MESSAGE("EnumClassUnspecified", EnumClassUnspecified != nullptr && EnumClassUnspecified->GetUnderlyingProperty()->IsA(FIntProperty::StaticClass()));
	FEnumProperty* EnumClassInt64 = CastField<FEnumProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, EnumClassInt64)));
	CHECK_MESSAGE("EnumClassInt64", EnumClassInt64 != nullptr && EnumClassInt64->GetUnderlyingProperty()->IsA(FInt64Property::StaticClass()));
	FEnumProperty* EnumClassUInt64 = CastField<FEnumProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, EnumClassUInt64)));
	CHECK_MESSAGE("EnumClassUInt64", EnumClassUInt64 != nullptr && EnumClassUInt64->GetUnderlyingProperty()->IsA(FUInt64Property::StaticClass()));

	FByteProperty* NamespacedInt8AsByte = CastField<FByteProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, NamespacedInt8AsByte)));
	CHECK_MESSAGE("NamespacedInt8AsByte", NamespacedInt8AsByte != nullptr);
	FByteProperty* NamespacedUInt8AsByte = CastField<FByteProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, NamespacedUInt8AsByte)));
	CHECK_MESSAGE("NamespacedUInt8AsByte", NamespacedUInt8AsByte != nullptr);
	FByteProperty* NamespacedEnumIntAsByte = CastField<FByteProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, NamespacedEnumIntAsByte)));
	CHECK_MESSAGE("NamespacedEnumIntAsByte", NamespacedEnumIntAsByte != nullptr);
	FByteProperty* NamespacedInt64AsByte = CastField<FByteProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, NamespacedInt64AsByte)));
	CHECK_MESSAGE("NamespacedInt64AsByte", NamespacedInt64AsByte != nullptr);
	FByteProperty* NamespacedEnumUInt64 = CastField<FByteProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, NamespacedEnumUInt64)));
	CHECK_MESSAGE("NamespacedEnumUInt64", NamespacedEnumUInt64 != nullptr);

	FByteProperty* RegularInt8AsByte = CastField<FByteProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, RegularInt8AsByte)));
	CHECK_MESSAGE("RegularInt8AsByte", RegularInt8AsByte != nullptr);
	FByteProperty* RegularUInt8AsByte = CastField<FByteProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, RegularUInt8AsByte)));
	CHECK_MESSAGE("RegularUInt8AsByte", RegularUInt8AsByte != nullptr);
	FByteProperty* RegularEnumIntAsByte = CastField<FByteProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, RegularEnumIntAsByte)));
	CHECK_MESSAGE("RegularEnumIntAsByte", RegularEnumIntAsByte != nullptr);
	FByteProperty* RegularEnumInt64AsByte = CastField<FByteProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, RegularEnumInt64AsByte)));
	CHECK_MESSAGE("RegularEnumInt64AsByte", RegularEnumInt64AsByte != nullptr);
	FByteProperty* RegularEnumUInt64AsByte = CastField<FByteProperty>(Obj->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTestEnum, RegularEnumUInt64AsByte)));
	CHECK_MESSAGE("RegularEnumUInt64AsByte", RegularEnumUInt64AsByte != nullptr);
}

TEST_CASE("CoreUObject::UEnum::UnderlyingType", "[CoreUObject]")
{
	CHECK_MESSAGE("ETestEnumClassInt8", StaticEnum<ETestEnumClassInt8>()->GetUnderlyingType() == UEnum::EUnderlyingType::int8);
	CHECK_MESSAGE("ETestEnumClassUInt8", StaticEnum<ETestEnumClassUInt8>()->GetUnderlyingType() == UEnum::EUnderlyingType::uint8);
	CHECK_MESSAGE("ETestEnumClassInt", StaticEnum<ETestEnumClassInt>()->GetUnderlyingType() == UEnum::EUnderlyingType::int32);
	CHECK_MESSAGE("ETestEnumClassUnspecified", StaticEnum<ETestEnumClassUnspecified>()->GetUnderlyingType() == UEnum::EUnderlyingType::int32);
	CHECK_MESSAGE("ETestEnumClassInt64", StaticEnum<ETestEnumClassInt64>()->GetUnderlyingType() == UEnum::EUnderlyingType::int64);
	CHECK_MESSAGE("ETestEnumClassUInt64", StaticEnum<ETestEnumClassUInt64>()->GetUnderlyingType() == UEnum::EUnderlyingType::uint64);

	CHECK_MESSAGE("ETestNamespacedEnumInt8", StaticEnum<ETestNamespacedEnumInt8::Type>()->GetUnderlyingType() == UEnum::EUnderlyingType::int8);
	CHECK_MESSAGE("ETestNamespacedEnumUInt8", StaticEnum<ETestNamespacedEnumUInt8::Type>()->GetUnderlyingType() == UEnum::EUnderlyingType::uint8);
	CHECK_MESSAGE("ETestNamespacedEnumInt", StaticEnum<ETestNamespacedEnumInt::Type>()->GetUnderlyingType() == UEnum::EUnderlyingType::int32);
	CHECK_MESSAGE("ETestNamespacedEnumInt64", StaticEnum<ETestNamespacedEnumInt64::Type>()->GetUnderlyingType() == UEnum::EUnderlyingType::int64);
	CHECK_MESSAGE("ETestNamespacedEnumUInt64", StaticEnum<ETestNamespacedEnumUInt64::Type>()->GetUnderlyingType() == UEnum::EUnderlyingType::uint64);

	CHECK_MESSAGE("ETestRegularInt8", StaticEnum<ETestRegularInt8>()->GetUnderlyingType() == UEnum::EUnderlyingType::int8);
	CHECK_MESSAGE("ETestRegularUInt8", StaticEnum<ETestRegularUInt8>()->GetUnderlyingType() == UEnum::EUnderlyingType::uint8);
	CHECK_MESSAGE("ETestRegularEnumInt", StaticEnum<ETestRegularEnumInt>()->GetUnderlyingType() == UEnum::EUnderlyingType::int32);
	CHECK_MESSAGE("ETestRegularEnumInt64", StaticEnum<ETestRegularEnumInt64>()->GetUnderlyingType() == UEnum::EUnderlyingType::int64);
	CHECK_MESSAGE("ETestRegularEnumUInt64", StaticEnum<ETestRegularEnumUInt64>()->GetUnderlyingType() == UEnum::EUnderlyingType::uint64);
	
}