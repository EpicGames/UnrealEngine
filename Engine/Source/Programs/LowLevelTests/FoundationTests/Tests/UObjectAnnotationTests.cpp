// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#include "TestCommon/Expectations.h"
#include "TestHarness.h"
#include "TestObject.h"
#include "UObject/Class.h"
#include "UObject/GCObject.h"
#include "UObject/Package.h"
#include "UObject/UObjectAnnotation.h"
#include "Tasks/Task.h"
#include "Async/ParallelFor.h"
#include "Async/ManualResetEvent.h"

TEST_CASE("UObjectAnnotation::ThreadSafety", "[UObjectAnnotation]")
{
	FUObjectAnnotationSparseBool AnnotationTest;
	FGCObject::StaticInit();

	UE::FManualResetEvent DoneEvent;
	UE::Tasks::FTask WatchdogTask =
		UE::Tasks::Launch(
			TEXT("WatchdogTask"),
			[&DoneEvent]()
			{
				double StartTime = FPlatformTime::Seconds();
				while (!DoneEvent.WaitFor(UE::FMonotonicTimeSpan::FromSeconds(1)))
				{
					if (FPlatformTime::Seconds() - StartTime > 60.0f)
					{
						UE_LOGF(LogTemp, Fatal, "UObjectAnnotation::ThreadSafety is non responsive and likely dead-locked. Aborting program.");
					}
				}
			}
		);

	ParallelFor(16,
		[&AnnotationTest](int32)
		{
			TStrongObjectPtr<UPackage> Package = nullptr;
			{
				FGCScopeGuard Guard;
				Package = TStrongObjectPtr<UPackage>(NewObject<UPackage>());
			}

			for (int32 Index = 0; Index < 1000; ++Index)
			{
				AnnotationTest.Set(Package.Get());
				CHECK(AnnotationTest.Get(Package.Get()));

				FPlatformProcess::YieldCycles(FMath::RandRange(0, 10000));

				if (IsInGameThread())
				{
					UObject* OldPackage = Package.Get();

					// Replace by a new object and garbage collect the old one
					Package = TStrongObjectPtr<UPackage>(NewObject<UPackage>());
					CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

					// Garbage collection should have unregistered the annotation automatically.
					CHECK(!AnnotationTest.Get(OldPackage));
				}
				else
				{
					AnnotationTest.Clear(Package.Get());
					CHECK(!AnnotationTest.Get(Package.Get()));
				}

				FPlatformProcess::YieldCycles(FMath::RandRange(0, 10000));
			}

			Package->ClearInternalFlags(EInternalObjectFlags::Async);
		}
	);

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	DoneEvent.Notify();
	WatchdogTask.Wait();
}
