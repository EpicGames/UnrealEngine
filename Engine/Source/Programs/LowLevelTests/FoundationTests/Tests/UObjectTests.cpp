// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#include "TestCommon/Expectations.h"
#include "TestHarness.h"
#include "TestObject.h"
#include "UObject/Class.h"
#include "UObject/Package.h"


TEST_CASE("CoreUObject::UFunction::Basic", "[CoreUObject]")
{
    FName PackageName = MakeUniqueObjectName(nullptr, UPackage::StaticClass(), FName("/Memory/UFunctionBasic"));
    UPackage* Package = NewObject<UPackage>(nullptr, UPackage::StaticClass(), PackageName);
    UTestObject* Obj = NewObject<UTestObject>(Package, "TestObject");

    int32 Value = Obj->GetBPOverrideableValue();
    int32 Value2 = Obj->GetBPOverrideableValue_Implementation();
    CHECK_EQUAL(Value, Value2);
}

TEST_CASE("CoreUObject::FProperty::Basic", "[CoreUObject]")
{
    FProperty* StrongObjectProp = UTestObject::StaticClass()->FindPropertyByName("StrongObjectReference");
    REQUIRE(StrongObjectProp != nullptr);
    CHECK_EQUAL(FName("OnRep_StrongObjectReference"), StrongObjectProp->RepNotifyFunc);
}

TEST_CASE("CoreUObject::Interface::Basic", "[CoreUObject]")
{
    UClass* TestObjectClass = UTestObject::StaticClass();
    CHECK(TestObjectClass->ImplementsInterface(UTestInterface::StaticClass()));

    FName PackageName = MakeUniqueObjectName(nullptr, UPackage::StaticClass(), FName("/Memory/InterfaceBasic"));
    UPackage* Package = NewObject<UPackage>(nullptr, UPackage::StaticClass(), PackageName);
    UTestObject* Obj = NewObject<UTestObject>(Package, "TestObject");

    ITestInterface* InterfaceObj = Cast<ITestInterface>(Obj);
    CHECK(InterfaceObj != nullptr);

    struct FParams
    {
        FName BaseName = "TestName";
        int32 Number = 17;
        FName Result;
    } Params;
    CHECK_EQUALS("Native call", 
        InterfaceObj->GetNumberedName(Params.BaseName, Params.Number),
        FName(Params.BaseName, Params.Number));

    UFunction* Function = UTestInterface::StaticClass()->FindFunctionByName("GetNumberedName");
    CHECK(Function != nullptr);
    Obj->ProcessEvent(Function, &Params);
    CHECK_EQUALS("BP call", Params.Result, FName(Params.BaseName, Params.Number));
}

TEST_CASE("CoreUObject::Class::SuperStructIterator", "[CoreUObject]")
{
    UClass* TestObjectClass = UTestObject::StaticClass();
    TArray<UClass*> Supers;
    for (UStruct* Struct : TestObjectClass->GetSuperStructIterator())
    {
        Supers.Add(CastChecked<UClass>(Struct));
    }
    REQUIRE_MESSAGE("Two supers", Supers.Num() == 2);
    CHECK_MESSAGE("Self first", Supers[0] == TestObjectClass);
    CHECK_MESSAGE("UObject second", Supers[1] == UObject::StaticClass());
}