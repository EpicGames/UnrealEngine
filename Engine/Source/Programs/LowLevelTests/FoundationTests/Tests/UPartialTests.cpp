// Copyright Epic Games, Inc. All Rights Reserved.

#include "TestHarness.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"
#include "UObject/UObjectPartials.h"
#include "UPartialTestClasses.h"
#include "UPartialTestPartials.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"

TEST_CASE("Partials BasicAccess", "[CoreUObject][Partials][UHT]")
{
	CollectGarbage(RF_NoFlags);

	UTestPartialObject* TestObj = NewObject<UTestPartialObject>();
	CHECK(TestObj != nullptr);

	SECTION("Native property accessible")
	{
		CHECK(TestObj->NativeValue == 100);
	}
	SECTION("Partial accessible via GetPartial")
	{
		FTestObjectPartial& Partial = TestObj->GetPartial<FTestObjectPartial>();
		CHECK(Partial.PartialValue == 43);
		CHECK(Partial.PartialString == TEXT("PartialDefault"));
		CHECK(Partial.PartialArray.Num() == 3);
	}
	SECTION("Second partial accessible")
	{
		FSecondObjectPartial& Partial = TestObj->GetPartial<FSecondObjectPartial>();
		CHECK(Partial.PartialFloat == 3.14f);
		CHECK(Partial.bPartialBool == true);
	}
	SECTION("Partial properties can be modified")
	{
		FTestObjectPartial& Partial = TestObj->GetPartial<FTestObjectPartial>();
		Partial.PartialValue = 999;
		Partial.PartialString = TEXT("Modified");
	}

	TestObj->AddToRoot();
	CollectGarbage(RF_NoFlags);
	TestObj->RemoveFromRoot();  
	TWeakObjectPtr<UTestPartialObject> Weak = TestObj;
	CollectGarbage(RF_NoFlags);
	CHECK(!Weak.IsValid());
}

TEST_CASE("Partials Inheritance", "[CoreUObject][Partials][UHT]")
{
	UDerivedTestPartialObject* DerivedObj = NewObject<UDerivedTestPartialObject>();

	SECTION("Native properties from base and derived accessible")
	{
		CHECK(DerivedObj->NativeValue == 100);
		CHECK(DerivedObj->DerivedValue == 200);
	}
	SECTION("Base class partials accessible from derived instance")
	{
		FTestObjectPartial& Partial = DerivedObj->GetPartial<FTestObjectPartial>();
		CHECK(Partial.PartialValue == 43);
		CHECK(Partial.PartialString == TEXT("PartialDefault"));
		CHECK(Partial.PartialArray.Num() == 3);
		CHECK(Partial.PartialArray[0] == 1);
		CHECK(Partial.PartialArray[1] == 2);
		CHECK(Partial.PartialArray[2] == 3);

		FSecondObjectPartial& SecondPartial = DerivedObj->GetPartial<FSecondObjectPartial>();
		CHECK(SecondPartial.PartialFloat == 3.14f);
	}
	SECTION("Derived class partial accessible")
	{
		FDerivedObjectPartial& DerivedPartial = DerivedObj->GetPartial<FDerivedObjectPartial>();
		CHECK(DerivedPartial.DerivedPartialValue == 999);
	}
}

TEST_CASE("Partial Properties", "[CoreUObject][Partials][UHT]")
{
	UTestPartialObject* TestObj = NewObject<UTestPartialObject>();
	UClass* TestClass = TestObj->GetClass();

	SECTION("Partial properties are in class property list")
	{
		CHECK(TestClass->FindPropertyByName(FName("PartialValue")) != nullptr);
		CHECK(TestClass->FindPropertyByName(FName("PartialString")) != nullptr);
		CHECK(TestClass->FindPropertyByName(FName("PartialArray")) != nullptr);
	}

	SECTION("Partial properties have correct types")
	{
		CHECK(TestClass->FindPropertyByName(FName("PartialValue"))->IsA<FIntProperty>());
		CHECK(TestClass->FindPropertyByName(FName("PartialString"))->IsA<FStrProperty>());
		CHECK(TestClass->FindPropertyByName(FName("PartialArray"))->IsA<FArrayProperty>());
	}

	SECTION("Partial properties can be accessed via property system")
	{
		FProperty* PartialValueProp = TestClass->FindPropertyByName(FName("PartialValue"));
		CHECK(PartialValueProp);
		void* ValuePtr = PartialValueProp->ContainerPtrToValuePtr<void>(TestObj);
		int32 Value = *(int32*)ValuePtr;
		CHECK(Value == 43);
		*(int32*)ValuePtr = 777;
		CHECK(TestObj->GetPartial<FTestObjectPartial>().PartialValue == 777);
	}
}

TEST_CASE("Partial AdvancedContainers", "[CoreUObject][Partials][UHT]")
{
	UTestPartialObject* TestObj = NewObject<UTestPartialObject>();

	SECTION("Map partial property accessible")
	{
		FAdvancedContainerPartial& Partial = TestObj->GetPartial<FAdvancedContainerPartial>();
		CHECK(Partial.PartialMap.Num() == 2);
		CHECK(Partial.PartialMap.Contains(1));
		CHECK(Partial.PartialMap.Contains(2));
		CHECK(Partial.PartialMap[1] == TEXT("One"));
		CHECK(Partial.PartialMap[2] == TEXT("Two"));
	}

	SECTION("Set partial property accessible")
	{
		FAdvancedContainerPartial& Partial = TestObj->GetPartial<FAdvancedContainerPartial>();
		CHECK(Partial.PartialSet.Num() == 3);
		CHECK(Partial.PartialSet.Contains(10));
		CHECK(Partial.PartialSet.Contains(20));
		CHECK(Partial.PartialSet.Contains(30));
	}

	SECTION("Enum partial property accessible")
	{
		FAdvancedContainerPartial& Partial = TestObj->GetPartial<FAdvancedContainerPartial>();
		CHECK(Partial.PartialEnum == ETestPartialEnum::ValueA);
		Partial.PartialEnum = ETestPartialEnum::ValueC;
		CHECK(TestObj->GetPartial<FAdvancedContainerPartial>().PartialEnum == ETestPartialEnum::ValueC);
	}

	SECTION("Map property is in class property list")
	{
		FProperty* MapProp = TestObj->GetClass()->FindPropertyByName(FName("PartialMap"));
		CHECK(MapProp != nullptr);
		CHECK(MapProp->IsA<FMapProperty>());
	}

	SECTION("Set property is in class property list")
	{
		FProperty* SetProp = TestObj->GetClass()->FindPropertyByName(FName("PartialSet"));
		CHECK(SetProp != nullptr);
		CHECK(SetProp->IsA<FSetProperty>());
	}

	SECTION("Enum property is in class property list")
	{
		UClass* TestClass = TestObj->GetClass();
		FProperty* EnumProp = TestClass->FindPropertyByName(FName("PartialEnum"));
		CHECK(EnumProp != nullptr);
		CHECK(EnumProp->IsA<FEnumProperty>());
	}
}

TEST_CASE("Partial PropertySerialization", "[CoreUObject][Partials][UHT]")
{
	UTestPartialObject* TestObj = NewObject<UTestPartialObject>();

	FTestObjectPartial& Partial = TestObj->GetPartial<FTestObjectPartial>();
	Partial.PartialValue = 12345;
	Partial.PartialString = TEXT("SerializedValue");
	Partial.PartialArray.Empty();
	Partial.PartialArray.Add(99);
	Partial.PartialArray.Add(88);
	Partial.PartialArray.Add(77);

	SECTION("Partial properties can be serialized via property system")
	{
		UClass* TestClass = TestObj->GetClass();

		// Serialize int property
		FProperty* IntProp = TestClass->FindPropertyByName(FName("PartialValue"));
		CHECK(IntProp != nullptr);

		TArray<uint8> IntData;
		FMemoryWriter IntWriter(IntData);
		void* IntValuePtr = IntProp->ContainerPtrToValuePtr<void>(TestObj);
		IntProp->SerializeItem(FStructuredArchiveFromArchive(IntWriter).GetSlot(), IntValuePtr);
		CHECK(IntData.Num() > 0);

		// Deserialize to verify
		FMemoryReader IntReader(IntData);
		int32 DeserializedInt = 0;
		IntProp->SerializeItem(FStructuredArchiveFromArchive(IntReader).GetSlot(), &DeserializedInt);
		CHECK(DeserializedInt == 12345);
	}

	SECTION("Partial string properties can be serialized")
	{
		UClass* TestClass = TestObj->GetClass();
		FProperty* StringProp = TestClass->FindPropertyByName(FName("PartialString"));
		CHECK(StringProp != nullptr);

		TArray<uint8> StringData;
		FMemoryWriter StringWriter(StringData);
		void* StringValuePtr = StringProp->ContainerPtrToValuePtr<void>(TestObj);
		StringProp->SerializeItem(FStructuredArchiveFromArchive(StringWriter).GetSlot(), StringValuePtr);
		CHECK(StringData.Num() > 0);
	}

	SECTION("Partial array properties can be serialized")
	{
		UClass* TestClass = TestObj->GetClass();
		FProperty* ArrayProp = TestClass->FindPropertyByName(FName("PartialArray"));
		CHECK(ArrayProp != nullptr);

		TArray<uint8> ArrayData;
		FMemoryWriter ArrayWriter(ArrayData);
		void* ArrayValuePtr = ArrayProp->ContainerPtrToValuePtr<void>(TestObj);
		ArrayProp->SerializeItem(FStructuredArchiveFromArchive(ArrayWriter).GetSlot(), ArrayValuePtr);
		CHECK(ArrayData.Num() > 0);
	}
}

TEST_CASE("Partial AdvancedTypes", "[CoreUObject][Partials][UHT]")
{
	UTestPartialObject* TestObj = NewObject<UTestPartialObject>();
	CHECK(TestObj != nullptr);

	SECTION("UClass* Partial property accessible")
	{
		FAdvancedTypesPartial& Partial = TestObj->GetPartial<FAdvancedTypesPartial>();
		CHECK(Partial.PartialClass == nullptr);
		Partial.PartialClass = UObject::StaticClass();
		CHECK(Partial.PartialClass == UObject::StaticClass());
		CHECK(TestObj->GetPartial<FAdvancedTypesPartial>().PartialClass == UObject::StaticClass());
	}

	SECTION("uint16 Partial property accessible")
	{
		FAdvancedTypesPartial& Partial = TestObj->GetPartial<FAdvancedTypesPartial>();
		CHECK(Partial.PartialUInt16 == 12345);
		Partial.PartialUInt16 = 54321;
	}

	SECTION("uint64 Partial property accessible")
	{
		FAdvancedTypesPartial& Partial = TestObj->GetPartial<FAdvancedTypesPartial>();
		CHECK(Partial.PartialUInt64 == 9876543210);
		Partial.PartialUInt64 = 1234567890123;
	}

	SECTION("Delegate Partial property accessible")
	{
		FAdvancedTypesPartial& Partial = TestObj->GetPartial<FAdvancedTypesPartial>();
		CHECK(!Partial.PartialDelegate.IsBound());
	}

	SECTION("Class property is in class property list")
	{
		UClass* TestClass = TestObj->GetClass();
		FProperty* ClassProp = TestClass->FindPropertyByName(FName("PartialClass"));
		CHECK(ClassProp != nullptr);
		CHECK(ClassProp->IsA<FClassProperty>());
	}

	SECTION("uint16 property is in class property list")
	{
		UClass* TestClass = TestObj->GetClass();
		FProperty* UInt16Prop = TestClass->FindPropertyByName(FName("PartialUInt16"));
		CHECK(UInt16Prop != nullptr);
		CHECK(UInt16Prop->IsA<FUInt16Property>());
	}

	SECTION("uint64 property is in class property list")
	{
		UClass* TestClass = TestObj->GetClass();
		FProperty* UInt64Prop = TestClass->FindPropertyByName(FName("PartialUInt64"));
		CHECK(UInt64Prop != nullptr);
		CHECK(UInt64Prop->IsA<FUInt64Property>());
	}

	SECTION("Delegate property is in class property list")
	{
		UClass* TestClass = TestObj->GetClass();
		FProperty* DelegateProp = TestClass->FindPropertyByName(FName("PartialDelegate"));
		CHECK(DelegateProp != nullptr);
		CHECK(DelegateProp->IsA<FDelegateProperty>());
	}

	SECTION("Multicast delegate property is in class property list")
	{
		UClass* TestClass = TestObj->GetClass();
		FProperty* MulticastProp = TestClass->FindPropertyByName(FName("PartialMulticastDelegate"));
		CHECK(MulticastProp != nullptr);
		CHECK(MulticastProp->IsA<FMulticastDelegateProperty>());
	}
}

TEST_CASE("Partial GetOwner", "[CoreUObject][Partials][GetOwner]")
{
	UTestPartialObject* TestObj = NewObject<UTestPartialObject>();
	TestObj->NativeValue = 12345;

	SECTION("GetOwner() returns correct object reference")
	{
		FTestObjectPartial& Partial = TestObj->GetPartial<FTestObjectPartial>();
		UTestPartialObject& Owner = Partial.GetOwner();
		CHECK(&Owner == TestObj);
		CHECK(Owner.NativeValue == 12345);
	}

	SECTION("GetOwner() const returns correct object reference")
	{
		const FTestObjectPartial& Partial = TestObj->GetPartial<FTestObjectPartial>();
		const UTestPartialObject& Owner = Partial.GetOwner();
		CHECK(&Owner == TestObj);
		CHECK(Owner.NativeValue == 12345);
	}

	SECTION("GetOwner() allows modifying owner properties")
	{
		FTestObjectPartial& Partial = TestObj->GetPartial<FTestObjectPartial>();
		Partial.GetOwner().NativeValue = 99999;
		CHECK(TestObj->NativeValue == 99999);
	}

	SECTION("Multiple Partials can access same owner")
	{
		FTestObjectPartial& Partial1 = TestObj->GetPartial<FTestObjectPartial>();
		FSecondObjectPartial& Partial2 = TestObj->GetPartial<FSecondObjectPartial>();
		CHECK(&Partial1.GetOwner() == TestObj);
		CHECK(&Partial2.GetOwner() == TestObj);
		CHECK(&Partial1.GetOwner() == &Partial2.GetOwner());
	}

	SECTION("GetOwner() works with derived classes")
	{
		UDerivedTestPartialObject* DerivedObj = NewObject<UDerivedTestPartialObject>();
		DerivedObj->NativeValue = 111;
		DerivedObj->DerivedValue = 222;
		FDerivedObjectPartial& DerivedPartial = DerivedObj->GetPartial<FDerivedObjectPartial>();
		UDerivedTestPartialObject& Owner = DerivedPartial.GetOwner();
		CHECK(&Owner == DerivedObj);
		CHECK(Owner.NativeValue == 111);
		CHECK(Owner.DerivedValue == 222);
	}
}

TEST_CASE("Partials UFunctions", "[CoreUObject][Partials][UFunctions]")
{
	UTestPartialObject* TestObj = NewObject<UTestPartialObject>();
	TestObj->NativeValue = 100;

	SECTION("Functions are discoverable via FindFunction")
	{
		UClass* Class = TestObj->GetClass();
		CHECK(Class->FindFunctionByName(FName("IncrementCounter")));
		CHECK(Class->FindFunctionByName(FName("GetCounter")));
		CHECK(Class->FindFunctionByName(FName("AddNumbers")));
		CHECK(Class->FindFunctionByName(FName("SetOwnerValue")));
	}

	SECTION("Function with no parameters can be called via ProcessEvent")
	{
		FTestFunctionPartial& Partial = TestObj->GetPartial<FTestFunctionPartial>();
		CHECK(Partial.FunctionCallCount == 0);
		UFunction* IncrementFunc = TestObj->GetClass()->FindFunctionByName(FName("IncrementCounter"));
		CHECK(IncrementFunc != nullptr);
		TestObj->ProcessEvent(IncrementFunc, nullptr);
		CHECK(Partial.FunctionCallCount == 1);
		TestObj->ProcessEvent(IncrementFunc, nullptr);
		CHECK(Partial.FunctionCallCount == 2);
	}

	SECTION("Function with return value can be called via ProcessEvent")
	{
		FTestFunctionPartial& Partial = TestObj->GetPartial<FTestFunctionPartial>();
		CHECK(Partial.FunctionCallCount == 0);
		Partial.FunctionCallCount = 42;
		UFunction* GetCounterFunc = TestObj->GetClass()->FindFunctionByName(FName("GetCounter"));
		CHECK(GetCounterFunc != nullptr);
		struct { int32 ReturnValue; } Params;
		TestObj->ProcessEvent(GetCounterFunc, &Params);
		CHECK(Params.ReturnValue == 42);
	}

	SECTION("Function with parameters can be called via ProcessEvent")
	{
		UFunction* AddNumbersFunc = TestObj->GetClass()->FindFunctionByName(FName("AddNumbers"));
		CHECK(AddNumbersFunc != nullptr);
		struct { int32 A; int32 B; int32 ReturnValue; } Params;
		Params.A = 15;
		Params.B = 27;
		Params.ReturnValue = 0;
		TestObj->ProcessEvent(AddNumbersFunc, &Params);
		CHECK(Params.ReturnValue == 42);
	}

	SECTION("Function can modify owner via GetOwner()")
	{
		CHECK(TestObj->NativeValue == 100);
		UFunction* SetOwnerValueFunc = TestObj->GetClass()->FindFunctionByName(FName("SetOwnerValue"));
		CHECK(SetOwnerValueFunc != nullptr);
		struct { int32 NewValue; } Params;
		Params.NewValue = 999;
		TestObj->ProcessEvent(SetOwnerValueFunc, &Params);
		CHECK(TestObj->NativeValue == 999);
	}
}

TEST_CASE("Partials OutlinedUFunctions", "[CoreUObject][Partials][UFunctions][Outlined]")
{
	UTestPartialObject* TestObj = NewObject<UTestPartialObject>();
	TestObj->NativeValue = 200;

	SECTION("Outlined Partial functions are discoverable via FindFunction")
	{
		UClass* Class = TestObj->GetClass();
		CHECK(Class->FindFunctionByName(FName("GetStoredValue")));
		CHECK(Class->FindFunctionByName(FName("SetStoredValue")));
		CHECK(Class->FindFunctionByName(FName("MultiplyStoredValue")));
		CHECK(Class->FindFunctionByName(FName("GetOwnerNativeValue")));
	}

	SECTION("Outlined Partial function with return value can be called via ProcessEvent")
	{
		FTestOutlinedFunctionPartial& Partial = TestObj->GetPartial<FTestOutlinedFunctionPartial>();
		CHECK(Partial.StoredValue == 42);
		UFunction* GetStoredValueFunc = TestObj->GetClass()->FindFunctionByName(FName("GetStoredValue"));
		CHECK(GetStoredValueFunc != nullptr);
		struct { int32 ReturnValue; } Params;
		TestObj->ProcessEvent(GetStoredValueFunc, &Params);
		CHECK(Params.ReturnValue == 42);
	}

	SECTION("Outlined Partial function with parameter can modify Partial state")
	{
		FTestOutlinedFunctionPartial& Partial = TestObj->GetPartial<FTestOutlinedFunctionPartial>();
		CHECK(Partial.StoredValue == 42);
		UFunction* SetStoredValueFunc = TestObj->GetClass()->FindFunctionByName(FName("SetStoredValue"));
		CHECK(SetStoredValueFunc != nullptr);
		struct { int32 NewValue; } Params;
		Params.NewValue = 123;
		TestObj->ProcessEvent(SetStoredValueFunc, &Params);
		CHECK(Partial.StoredValue == 123);
	}

	SECTION("Outlined Partial function with parameter and return value works via ProcessEvent")
	{
		FTestOutlinedFunctionPartial& Partial = TestObj->GetPartial<FTestOutlinedFunctionPartial>();
		Partial.StoredValue = 10;
		UFunction* MultiplyFunc = TestObj->GetClass()->FindFunctionByName(FName("MultiplyStoredValue"));
		CHECK(MultiplyFunc != nullptr);
		struct { int32 Multiplier; int32 ReturnValue; } Params;
		Params.Multiplier = 5;
		Params.ReturnValue = 0;
		TestObj->ProcessEvent(MultiplyFunc, &Params);
		CHECK(Params.ReturnValue == 50);
		CHECK(Partial.StoredValue == 50);
	}

	SECTION("Outlined Partial function can access owner via GetOwner()")
	{
		TestObj->NativeValue = 200;
		UFunction* GetOwnerNativeValueFunc = TestObj->GetClass()->FindFunctionByName(FName("GetOwnerNativeValue"));
		CHECK(GetOwnerNativeValueFunc != nullptr);
		struct { int32 ReturnValue; } Params;
		TestObj->ProcessEvent(GetOwnerNativeValueFunc, &Params);
		CHECK(Params.ReturnValue == 200);
	}
}

TEST_CASE("Partials BlueprintCallable", "[CoreUObject][Partials][UHT][Blueprint]")
{
	UBlueprintablePartialObject* TestObj = NewObject<UBlueprintablePartialObject>(GetTransientPackage(), "TestBPObject");

	SECTION("BlueprintCallable Partial functions are discoverable via FindFunction")
	{
		UClass* Class = UBlueprintablePartialObject::StaticClass();
		CHECK(Class->FindFunctionByName(FName("IncrementBPCounter")));
		CHECK(Class->FindFunctionByName(FName("GetBPCounter")));
		CHECK(Class->FindFunctionByName(FName("MultiplyBP")));
		CHECK(Class->FindFunctionByName(FName("SetOwnerBlueprintValue")));
		CHECK(Class->FindFunctionByName(FName("GetOwnerBlueprintValue")));
	}

	SECTION("BlueprintCallable Partial function has correct flags")
	{
		UFunction* IncrementFunc = TestObj->GetClass()->FindFunctionByName(FName("IncrementBPCounter"));
		CHECK(IncrementFunc->HasAnyFunctionFlags(FUNC_BlueprintCallable));
	}

	SECTION("BlueprintPure Partial function has correct flags")
	{
		UFunction* GetCounterFunc = TestObj->GetClass()->FindFunctionByName(FName("GetBPCounter"));
		CHECK(GetCounterFunc->HasAnyFunctionFlags(FUNC_BlueprintCallable));
		CHECK(GetCounterFunc->HasAnyFunctionFlags(FUNC_BlueprintPure));
	}

	SECTION("BlueprintCallable Partial function works via ProcessEvent")
	{
		FBlueprintFunctionPartial& Partial = TestObj->GetPartial<FBlueprintFunctionPartial>();
		CHECK(Partial.Counter == 0);
		UFunction* IncrementFunc = TestObj->GetClass()->FindFunctionByName(FName("IncrementBPCounter"));
		TestObj->ProcessEvent(IncrementFunc, nullptr);
		CHECK(Partial.Counter == 1);
		TestObj->ProcessEvent(IncrementFunc, nullptr);
		CHECK(Partial.Counter == 2);
	}

	SECTION("BlueprintPure Partial function with return value works via ProcessEvent")
	{
		FBlueprintFunctionPartial& Partial = TestObj->GetPartial<FBlueprintFunctionPartial>();
		Partial.Counter = 42;
		UFunction* GetCounterFunc = TestObj->GetClass()->FindFunctionByName(FName("GetBPCounter"));
		struct { int32 ReturnValue; } Params;
		Params.ReturnValue = 0;
		TestObj->ProcessEvent(GetCounterFunc, &Params);
		CHECK(Params.ReturnValue == 42);
	}

	SECTION("BlueprintCallable Partial function with parameters and return value works")
	{
		UFunction* MultiplyFunc = TestObj->GetClass()->FindFunctionByName(FName("MultiplyBP"));
		struct { int32 A; int32 B; int32 ReturnValue; } Params;
		Params.A = 7;
		Params.B = 8;
		Params.ReturnValue = 0;
		TestObj->ProcessEvent(MultiplyFunc, &Params);
		CHECK(Params.ReturnValue == 56);
	}

	SECTION("BlueprintCallable Partial function can access and modify owner")
	{
		TestObj->BlueprintValue = 100;
		UFunction* SetOwnerFunc = TestObj->GetClass()->FindFunctionByName(FName("SetOwnerBlueprintValue"));
		struct { int32 NewValue; } Params;
		Params.NewValue = 999;
		TestObj->ProcessEvent(SetOwnerFunc, &Params);
		CHECK(TestObj->BlueprintValue == 999);
	}

	SECTION("BlueprintPure Partial function can read owner value")
	{
		TestObj->BlueprintValue = 777;
		UFunction* GetOwnerFunc = TestObj->GetClass()->FindFunctionByName(FName("GetOwnerBlueprintValue"));
		struct { int32 ReturnValue; } Params;
		Params.ReturnValue = 0;
		TestObj->ProcessEvent(GetOwnerFunc, &Params);
		CHECK(Params.ReturnValue == 777);
	}
}

TEST_CASE("Partials Lifecycle", "[CoreUObject][Partials][UHT]")
{
	SECTION("Partial constructors are called for CDO")
	{
		UClass* TestClass = UTestPartialObject::StaticClass();
		UObject* CDO = TestClass->GetDefaultObject();
		CHECK(CDO != nullptr);
		UTestPartialObject* TestCDO = Cast<UTestPartialObject>(CDO);
		CHECK(TestCDO != nullptr);
		FTestObjectPartial& Partial = TestCDO->GetPartial<FTestObjectPartial>();
		CHECK(Partial.PartialValue == 43);
		CHECK(Partial.PartialString == TEXT("PartialDefault"));
		CHECK(Partial.PartialArray.Num() == 3);
		CHECK(Partial.PartialArray[0] == 1);
		CHECK(Partial.PartialArray[1] == 2);
		CHECK(Partial.PartialArray[2] == 3);
	}

	SECTION("Partial destructors are called when object is destroyed")
	{
		FName PackageName = MakeUniqueObjectName(nullptr, UPackage::StaticClass(), FName("/Memory/DestructorTest"));
		UPackage* Package = NewObject<UPackage>(nullptr, UPackage::StaticClass(), PackageName);
		{
			UTestPartialObject* TestObj = NewObject<UTestPartialObject>(Package, "TestObject");
			CHECK(TestObj != nullptr);
			FAdvancedContainerPartial& Partial = TestObj->GetPartial<FAdvancedContainerPartial>();
			Partial.PartialMap.Add(1, TEXT("Test"));
			Partial.PartialSet.Add(42);
		}
		Package->MarkAsGarbage();
	}
}

// ============================================================================
// Property Merging Tests - verify properties are added to UClass correctly
// ============================================================================

TEST_CASE("Partials PropertyMerging", "[CoreUObject][Partials][UHT]")
{
	UClass* TestClass = UTestPartialObject::StaticClass();

	SECTION("Partial properties added to UClass property list")
	{
		int32 PropertyCount = 0;
		for (FField* Field = TestClass->ChildProperties; Field; Field = Field->Next)
		{
			if (CastField<FProperty>(Field))
			{
				PropertyCount++;
			}
		}
		CHECK(PropertyCount > 4);

		bool bFoundPartialValue = false;
		bool bFoundPartialString = false;
		bool bFoundPartialArray = false;
		for (FField* Field = TestClass->ChildProperties; Field; Field = Field->Next)
		{
			if (FProperty* Prop = CastField<FProperty>(Field))
			{
				FString PropName = Prop->GetName();
				if (PropName == TEXT("PartialValue")) bFoundPartialValue = true;
				if (PropName == TEXT("PartialString")) bFoundPartialString = true;
				if (PropName == TEXT("PartialArray")) bFoundPartialArray = true;
			}
		}
		CHECK(bFoundPartialValue);
		CHECK(bFoundPartialString);
		CHECK(bFoundPartialArray);
	}

	SECTION("Partial properties can be found by name")
	{
		FProperty* PartialValueProp = TestClass->FindPropertyByName(FName("PartialValue"));
		CHECK(PartialValueProp != nullptr);
		CHECK(PartialValueProp->IsA<FIntProperty>());
		FProperty* PartialStringProp = TestClass->FindPropertyByName(FName("PartialString"));
		CHECK(PartialStringProp != nullptr);
		CHECK(PartialStringProp->IsA<FStrProperty>());
		FProperty* PartialArrayProp = TestClass->FindPropertyByName(FName("PartialArray"));
		CHECK(PartialArrayProp != nullptr);
		CHECK(PartialArrayProp->IsA<FArrayProperty>());
	}

	SECTION("Partial properties have correct offsets for reflection")
	{
		UTestPartialObject* TestObj = NewObject<UTestPartialObject>();
		FProperty* PartialValueProp = TestClass->FindPropertyByName(FName("PartialValue"));
		void* PropValuePtr = PartialValueProp->ContainerPtrToValuePtr<void>(TestObj);
		int32* ValuePtr = static_cast<int32*>(PropValuePtr);
		FTestObjectPartial& Partial = TestObj->GetPartial<FTestObjectPartial>();
		CHECK(*ValuePtr == Partial.PartialValue);
		CHECK(*ValuePtr == 43); // Default value from constructor
	}

	SECTION("Derived class includes base class Partial properties")
	{
		UClass* DerivedClass = UDerivedTestPartialObject::StaticClass();
		UClass* BaseClass = UTestPartialObject::StaticClass();
		FProperty* BasePartialValue = DerivedClass->FindPropertyByName(FName("PartialValue"));
		FProperty* DerivedPartialValue = DerivedClass->FindPropertyByName(FName("DerivedPartialValue"));
		CHECK(DerivedClass->GetPropertiesSize() > BaseClass->GetPropertiesSize());
	}
}

TEST_CASE("Partials ObjectDuplication", "[CoreUObject][Partials][UHT]")
{
	UTestPartialObject* SourceObj = NewObject<UTestPartialObject>(GetTransientPackage(), "SourceObject");
	SourceObj->NativeValue = 12345;
	FTestObjectPartial& SourcePartial = SourceObj->GetPartial<FTestObjectPartial>();
	SourcePartial.PartialValue = 99999;
	SourcePartial.PartialString = TEXT("DuplicationTest");
	UTestPartialObject* DuplicatedObj = NewObject<UTestPartialObject>(GetTransientPackage(), "DuplicatedObject");
	CHECK(DuplicatedObj != nullptr);
	CHECK(DuplicatedObj != SourceObj);
	DuplicatedObj->NativeValue = SourceObj->NativeValue;
	FTestObjectPartial& DupPartial = DuplicatedObj->GetPartial<FTestObjectPartial>();
	DupPartial.PartialValue = SourcePartial.PartialValue;
	DupPartial.PartialString = SourcePartial.PartialString;
}

TEST_CASE("Partials InheritanceWithoutPartialAttribute", "[CoreUObject][Partials][UHT]")
{
	UDerivedDerivedTestPartialObject* DerivedDerivedObj = NewObject<UDerivedDerivedTestPartialObject>(GetTransientPackage(), "DerivedDerivedObject");

	SECTION("Native properties from all levels accessible")
	{
		CHECK(DerivedDerivedObj->NativeValue == 100);
		CHECK(DerivedDerivedObj->DerivedValue == 200);
	}

	SECTION("Grandparent class Partials should be accessible (CURRENTLY FAILS)")
	{
		FTestObjectPartial& BasePartial = DerivedDerivedObj->GetPartial<FTestObjectPartial>();
		CHECK(BasePartial.PartialValue == 43);
		CHECK(BasePartial.PartialString == TEXT("PartialDefault"));
		CHECK(BasePartial.PartialArray.Num() == 3);
	}

	SECTION("Parent class Partials should be accessible (CURRENTLY FAILS)")
	{
		FDerivedObjectPartial& DerivedPartial = DerivedDerivedObj->GetPartial<FDerivedObjectPartial>();
		CHECK(DerivedPartial.DerivedPartialValue == 999);
	}

	SECTION("Partial properties should appear in reflection")
	{
		UClass* Class = DerivedDerivedObj->GetClass();
		FProperty* PartialValueProp = Class->FindPropertyByName(FName("PartialValue"));
		CHECK(PartialValueProp != nullptr);
		FProperty* DerivedPartialValueProp = Class->FindPropertyByName(FName("DerivedPartialValue"));
		CHECK(DerivedPartialValueProp != nullptr);
	}
}

// Some build targets aren't configured to support UPS in low-level test runs and will assert at runtime.
extern bool CanUseUnversionedPropertySerialization(const ITargetPlatform* Target);

TEST_CASE("Partials PartialProperties", "[CoreUObject][Serialization][Partials]")
{
	if (!CanUseUnversionedPropertySerialization(nullptr))
	{
		return;
	}

	USerializationTestObject* OriginalObj = NewObject<USerializationTestObject>();
	OriginalObj->NativeProperty = 123;
	OriginalObj->NativeString = TEXT("Modified");

	FSerializationTestPartial& Partial = OriginalObj->GetPartial<FSerializationTestPartial>();
	Partial.PartialInt = 456;
	Partial.PartialString = TEXT("ModifiedPartial");
	Partial.PartialArray = {10, 20, 30, 40};
	Partial.PartialOptional = 777;

	SECTION("Unversioned serialization preserves partial properties")
	{
		TArray<uint8> SaveData;
		FMemoryWriter Writer(SaveData);
		Writer.SetUseUnversionedPropertySerialization(true);
		OriginalObj->Serialize(Writer);
		USerializationTestObject* LoadedObj = NewObject<USerializationTestObject>();
		FMemoryReader Reader(SaveData);
		Reader.SetUseUnversionedPropertySerialization(true);
		LoadedObj->Serialize(Reader);
		CHECK(LoadedObj->NativeProperty == 123);
		CHECK(LoadedObj->NativeString == TEXT("Modified"));
		FSerializationTestPartial& LoadedPartial = LoadedObj->GetPartial<FSerializationTestPartial>();
		CHECK(LoadedPartial.PartialInt == 456);
		CHECK(LoadedPartial.PartialString == TEXT("ModifiedPartial"));
		CHECK(LoadedPartial.PartialArray.Num() == 4);
		CHECK(LoadedPartial.PartialArray[0] == 10);
		CHECK(LoadedPartial.PartialArray[3] == 40);
		CHECK(LoadedPartial.PartialOptional.IsSet());
		CHECK(LoadedPartial.PartialOptional.GetValue() == 777);
	}

	SECTION("Default values work correctly for partial properties")
	{
		USerializationTestObject* DefaultObj = NewObject<USerializationTestObject>();
		TArray<uint8> SaveData;
		FMemoryWriter Writer(SaveData);
		Writer.SetUseUnversionedPropertySerialization(true);
		DefaultObj->Serialize(Writer);
		USerializationTestObject* LoadedObj = NewObject<USerializationTestObject>();
		FMemoryReader Reader(SaveData);
		Reader.SetUseUnversionedPropertySerialization(true);
		LoadedObj->Serialize(Reader);
		FSerializationTestPartial& LoadedPartial = LoadedObj->GetPartial<FSerializationTestPartial>();
		CHECK(LoadedPartial.PartialInt == 42);
		CHECK(LoadedPartial.PartialString == TEXT("Partial"));
	}

	SECTION("Static arrays in partials serialize correctly")
	{
		USerializationTestObject* Obj = NewObject<USerializationTestObject>();
		FSerializationTestPartial& Partial2 = Obj->GetPartial<FSerializationTestPartial>();
		Partial2.PartialStaticArray[0] = 100;
		Partial2.PartialStaticArray[1] = 200;
		Partial2.PartialStaticArray[2] = 300;
		TArray<uint8> SaveData;
		FMemoryWriter Writer(SaveData);
		Writer.SetUseUnversionedPropertySerialization(true);
		Obj->Serialize(Writer);
		USerializationTestObject* LoadedObj = NewObject<USerializationTestObject>();
		FMemoryReader Reader(SaveData);
		Reader.SetUseUnversionedPropertySerialization(true);
		LoadedObj->Serialize(Reader);
		FSerializationTestPartial& LoadedPartial = LoadedObj->GetPartial<FSerializationTestPartial>();
		CHECK(LoadedPartial.PartialStaticArray[0] == 100);
		CHECK(LoadedPartial.PartialStaticArray[1] == 200);
		CHECK(LoadedPartial.PartialStaticArray[2] == 300);
	}
}

TEST_CASE("Partials InheritedPartialProperties", "[CoreUObject][Serialization][Partials]")
{
	if (!CanUseUnversionedPropertySerialization(nullptr))
	{
		return;
	}

	SECTION("Derived class serializes both base and derived partial properties")
	{
		UDerivedSerializationTestObject* OriginalObj = NewObject<UDerivedSerializationTestObject>();
		FSerializationTestPartial& BasePartial = OriginalObj->GetPartial<FSerializationTestPartial>();
		BasePartial.PartialInt = 111;
		FDerivedSerializationTestPartial& DerivedPartial = OriginalObj->GetPartial<FDerivedSerializationTestPartial>();
		DerivedPartial.DerivedPartialFloat = 2.71f;
		TArray<uint8> SaveData;
		FMemoryWriter Writer(SaveData);
		Writer.SetUseUnversionedPropertySerialization(true);
		OriginalObj->Serialize(Writer);
		UDerivedSerializationTestObject* LoadedObj = NewObject<UDerivedSerializationTestObject>();
		FMemoryReader Reader(SaveData);
		Reader.SetUseUnversionedPropertySerialization(true);
		LoadedObj->Serialize(Reader);
		FSerializationTestPartial& LoadedBasePartial = LoadedObj->GetPartial<FSerializationTestPartial>();
		CHECK(LoadedBasePartial.PartialInt == 111);
		FDerivedSerializationTestPartial& LoadedDerivedPartial = LoadedObj->GetPartial<FDerivedSerializationTestPartial>();
		CHECK(LoadedDerivedPartial.DerivedPartialFloat == 2.71f);
	}
}

TEST_CASE("Partials GetDefaultValue_NegativeOffset", "[CoreUObject][Serialization][Partials]")
{
	if (!CanUseUnversionedPropertySerialization(nullptr))
	{
		return;
	}

	USerializationTestObject* Obj = NewObject<USerializationTestObject>();
	FSerializationTestPartial& Partial = Obj->GetPartial<FSerializationTestPartial>();
	Partial.PartialString = TEXT("ModifiedValue");
	TArray<uint8> SaveData;
	FMemoryWriter Writer(SaveData);
	Writer.SetUseUnversionedPropertySerialization(true);
	FStructuredArchiveFromArchive Adapter(Writer);
	UClass* Class = Obj->GetClass();
	const void* Defaults = Class->GetDefaultObject();
	Class->SerializeTaggedProperties(Adapter.GetSlot(), (uint8*)Obj, Class, (uint8*)Defaults);
	CHECK(SaveData.Num() > 0);
}