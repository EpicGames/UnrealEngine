// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

public class MathCoreTests : TestModuleRules
{
	static MathCoreTests()
	{
		if (InTestMode)
		{
			TestMetadata = new Metadata();
			TestMetadata.TestName = "MathCore";
			TestMetadata.TestShortName = "MathCore";
			TestMetadata.ReportType = "testdata";
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Linux);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Mac);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.Android);
			TestMetadata.SupportedPlatforms.Add(UnrealTargetPlatform.IOS);

			string PlatformCompilationArgs;
			foreach (var Platform in UnrealTargetPlatform.GetValidPlatforms())
			{
				if (Platform == UnrealTargetPlatform.Android)
				{
					PlatformCompilationArgs = "-allmodules -architectures=arm64";
				}
				else
				{
					PlatformCompilationArgs = "-allmodules";
				}
				TestMetadata.PlatformCompilationExtraArgs.Add(Platform, PlatformCompilationArgs);
			}
			TestMetadata.PlatformsRunUnsupported.Remove(UnrealTargetPlatform.Android);
		}
	}

	public MathCoreTests(ReadOnlyTargetRules Target) : base(Target, InUsesCatch2:true)
	{
		PrivateDependencyModuleNames.AddRange(
			[
				"Core",
				"MathCore",
			]);

		if (Target.Platform == UnrealTargetPlatform.IOS)
		{
			PrivateDependencyModuleNames.AddRange([ "ApplicationCore", "CoreUObject" ]);
		}
	}
}