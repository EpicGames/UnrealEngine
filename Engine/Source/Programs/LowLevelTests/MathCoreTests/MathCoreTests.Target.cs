// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class MathCoreTestsTarget : TestTargetRules
{
	public MathCoreTestsTarget(TargetInfo Target) : base(Target)
	{
		// Collects all tests decorated with #if WITH_LOW_LEVELTESTS from dependencies
		// TODO [jonathan.bard] : disabled for now : we can enable this (and move the tests in MathCoreTests to a Tests sub-folder alongside the code they're validating), when https://jira.it.epicgames.com/browse/UE-205189 is implemented : 
		//  Without this, the tests from all linked modules (i.e. Core) would be run as part of this executable, which would be wasteful : 
		// bWithLowLevelTestsOverride = true;
	}
}
