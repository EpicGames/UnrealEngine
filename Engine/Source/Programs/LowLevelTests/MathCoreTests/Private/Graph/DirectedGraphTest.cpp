// Copyright Epic Games, Inc. All Rights Reserved.

// TODO [jonathan.bard] : don't use "#if WITH_LOW_LEVEL_TESTS" for now : we can enable this (and move the tests in MathCoreTests to a Tests sub-folder alongside the code they're validating), when https://jira.it.epicgames.com/browse/UE-205189 is implemented : 
//  Without this, the tests from all linked modules (i.e. Core) would be run as part of this executable, which would be wasteful : 
//#if WITH_LOW_LEVEL_TESTS

#include "CoreMinimal.h"
#include "TestHarness.h"

#include "Graph/DirectedGraphUtils.h"

SCENARIO("MathCore::Graph::DirectedGraph", "[ApplicationContextMask][SmokeFilter]")
{
	using namespace UE::MathCore::Graph;

	GIVEN("A graph without any strongly connected components")
	{
		TArray<int32> Vertices({0, 1, 2, 3, 4, 5, 6, 7});
		TSet<FDirectedEdge> Edges;

		Edges.Add(FDirectedEdge(0, 1));
		Edges.Add(FDirectedEdge(0, 2));
		Edges.Add(FDirectedEdge(1, 3));
		Edges.Add(FDirectedEdge(2, 3));
		Edges.Add(FDirectedEdge(3, 6));
		Edges.Add(FDirectedEdge(0, 6));
		Edges.Add(FDirectedEdge(6, 7));

		WHEN("the tarjan algorithm tries to find strongly connected components")
		{
			TArray<FStronglyConnectedComponent> OutComponents;
			bool bResult = TarjanStronglyConnectedComponents(Edges, OutComponents);

			THEN("the algorithm does not find any strongly connected components")
			{
				REQUIRE(bResult == false);
				REQUIRE(OutComponents.Num() == 0);
			}
		}

		WHEN("the graph is sorted depth first")
		{
			TArray<int32> Order;
			bool bSuccess = DepthFirstTopologicalSort(Vertices, Edges.Array(), Order);

			THEN("the algorithm succeeds")
			{
				REQUIRE(bSuccess);
			}

			THEN("the algorithm produces a valid order")
			{
				int32 Index0 = Order.Find(0);
				int32 Index1 = Order.Find(1);
				int32 Index2 = Order.Find(2);
				int32 Index3 = Order.Find(3);
				int32 Index4 = Order.Find(4);
				int32 Index5 = Order.Find(5);
				int32 Index6 = Order.Find(6);
				int32 Index7 = Order.Find(7);

				REQUIRE(Index0 >= 0);
				REQUIRE(Index1 >= 0);
				REQUIRE(Index2 >= 0);
				REQUIRE(Index3 >= 0);
				REQUIRE(Index4 >= 0);
				REQUIRE(Index5 >= 0);
				REQUIRE(Index6 >= 0);
				REQUIRE(Index7 >= 0);

				REQUIRE(Index0 < Index1);
				REQUIRE(Index0 < Index2);
				REQUIRE(Index0 < Index3);
				REQUIRE(Index0 < Index6);
				REQUIRE(Index1 < Index3);
				REQUIRE(Index1 < Index6);
				REQUIRE(Index2 < Index3);
				REQUIRE(Index2 < Index6);
				REQUIRE(Index6 < Index7);
			}
		}

		WHEN("the graph is sorted using kahns algorithm")
		{
			TArray<int32> Order;
			bool bSuccess = KahnTopologicalSort(Vertices, Edges.Array(), Order);

			THEN("the algorithm succeeds")
			{
				REQUIRE(bSuccess);
			}

			THEN("the algorithm produces a valid order")
			{
				int32 Index0 = Order.Find(0);
				int32 Index1 = Order.Find(1);
				int32 Index2 = Order.Find(2);
				int32 Index3 = Order.Find(3);
				int32 Index4 = Order.Find(4);
				int32 Index5 = Order.Find(5);
				int32 Index6 = Order.Find(6);
				int32 Index7 = Order.Find(7);

				REQUIRE(Index0 >= 0);
				REQUIRE(Index1 >= 0);
				REQUIRE(Index2 >= 0);
				REQUIRE(Index3 >= 0);
				REQUIRE(Index4 >= 0);
				REQUIRE(Index5 >= 0);
				REQUIRE(Index6 >= 0);
				REQUIRE(Index7 >= 0);

				REQUIRE(Index0 < Index1);
				REQUIRE(Index0 < Index2);
				REQUIRE(Index0 < Index3);
				REQUIRE(Index0 < Index6);
				REQUIRE(Index1 < Index3);
				REQUIRE(Index1 < Index6);
				REQUIRE(Index2 < Index3);
				REQUIRE(Index2 < Index6);
				REQUIRE(Index6 < Index7);
			}
		}

		WHEN("the tree is traversed depth first for nodes")
		{
			FDirectedTree Tree;
			BuildDirectedTree(Edges.Array(), Tree);

			TMap<int32, int32> VisitOrder;
			int32 VisitCounter = 0;

			DepthFirstNodeTraversal(0, Tree, [&](int32 Vertex) -> bool
				{
					VisitOrder.Add(Vertex, VisitCounter);
					VisitCounter++;
					
					if (Vertex == 6)
					{
						return false;
					} 
					return true;
				}
			);

			THEN("The vertices are visited in a valid order")
			{
				REQUIRE(VisitOrder.Contains(0));
				REQUIRE(VisitOrder.Contains(1));
				REQUIRE(VisitOrder.Contains(2));
				REQUIRE(VisitOrder.Contains(3));
				REQUIRE(!VisitOrder.Contains(4));
				REQUIRE(!VisitOrder.Contains(5));
				REQUIRE(VisitOrder.Contains(6));
				REQUIRE(!VisitOrder.Contains(7));

				int32 Order0 = VisitOrder[0];
				int32 Order1 = VisitOrder[1];
				int32 Order2 = VisitOrder[2];
				int32 Order3 = VisitOrder[3];
				int32 Order6 = VisitOrder[6];

				REQUIRE(Order0 >= 0);
				REQUIRE(Order1 >= 0);
				REQUIRE(Order2 >= 0);
				REQUIRE(Order3 >= 0);
				REQUIRE(Order6 >= 0);

				REQUIRE(Order0 < Order1);
				REQUIRE(Order0 < Order2);
				REQUIRE(Order0 < Order3);
				REQUIRE(Order0 < Order6);
				bool bValidOrder = (Order1 < Order3) || (Order2 < Order3);
				REQUIRE(bValidOrder);
			}
		}

		WHEN("the tree is traversed depth first for edges")
		{
			FDirectedTree Tree;
			BuildDirectedTree(Edges.Array(), Tree);

			TMap<TTuple<int32, int32>, int32> VisitOrder;
			int32 VisitCounter = 0;

			DepthFirstEdgeTraversal(0, Tree, [&](int32 SourceVertex, int32 DestinationVertex) -> bool
			{
				VisitOrder.Add(MakeTuple(SourceVertex, DestinationVertex), VisitCounter);
				VisitCounter++;

				// Stop at one of the edges : 
				if ((SourceVertex == 0) && (DestinationVertex == 1))
				{
					return false;
				}
				return true;
			}
			);

			THEN("The vertices are visited in a valid order")
			{
				REQUIRE(VisitOrder.Contains(MakeTuple(0, 6)));
				REQUIRE(VisitOrder.Contains(MakeTuple(6, 7)));
				REQUIRE(VisitOrder.Contains(MakeTuple(0, 2)));
				REQUIRE(VisitOrder.Contains(MakeTuple(2, 3)));
				REQUIRE(VisitOrder.Contains(MakeTuple(3, 6)));
				REQUIRE(VisitOrder.Contains(MakeTuple(0, 1)));
				REQUIRE(!VisitOrder.Contains(MakeTuple(1, 3))); // This edge should have been skipped when we stopped at (0, 1)
				REQUIRE(VisitOrder[MakeTuple(0, 6)] < VisitOrder[MakeTuple(6, 7)]);
				REQUIRE(VisitOrder[MakeTuple(0, 2)] < VisitOrder[MakeTuple(2, 3)]);
				REQUIRE(VisitOrder[MakeTuple(2, 3)] < VisitOrder[MakeTuple(3, 6)]);
			}
		}

		WHEN("the tree is traversed breadth first")
		{
			FDirectedTree Tree;
			BuildDirectedTree(Edges.Array(), Tree);

			TMap<int32, int32> VisitOrder;
			int32 VisitCounter = 0;

			BreadthFirstNodeTraversal(0, Tree, [&](int32 Vertex) -> bool
				{
					VisitOrder.Add(Vertex, VisitCounter);
					VisitCounter++;
					
					if (Vertex == 6)
					{
						return false;
					} 
					return true;
				}
			);

			THEN("The vertices are visited in a valid order")
			{
				REQUIRE(VisitOrder.Contains(0));
				REQUIRE(VisitOrder.Contains(1));
				REQUIRE(VisitOrder.Contains(2));
				REQUIRE(VisitOrder.Contains(3));
				REQUIRE(!VisitOrder.Contains(4));
				REQUIRE(!VisitOrder.Contains(5));
				REQUIRE(VisitOrder.Contains(6));
				REQUIRE(!VisitOrder.Contains(7));

				int32 Order0 = VisitOrder[0];
				int32 Order1 = VisitOrder[1];
				int32 Order2 = VisitOrder[2];
				int32 Order3 = VisitOrder[3];
				int32 Order6 = VisitOrder[6];

				REQUIRE(Order0 >= 0);
				REQUIRE(Order1 >= 0);
				REQUIRE(Order2 >= 0);
				REQUIRE(Order3 >= 0);
				REQUIRE(Order6 >= 0);

				REQUIRE(Order0 < Order1);
				REQUIRE(Order0 < Order2);
				REQUIRE(Order0 < Order3);
				REQUIRE(Order0 < Order6);
				REQUIRE(Order1 < Order3);
				REQUIRE(Order2 < Order3);
				REQUIRE(Order6 < Order3);
			}
		}

		WHEN("the tree is traversed breadth first for edges")
		{
			FDirectedTree Tree;
			BuildDirectedTree(Edges.Array(), Tree);

			TMap<TTuple<int32, int32>, int32> VisitOrder;
			int32 VisitCounter = 0;

			BreadthFirstEdgeTraversal(0, Tree, [&](int32 SourceVertex, int32 DestinationVertex) -> bool
			{
				VisitOrder.Add(MakeTuple(SourceVertex, DestinationVertex), VisitCounter);
				VisitCounter++;

				// Stop at one of the vertices : 
				if (DestinationVertex == 6)
				{
					return false;
				}
				return true;
			}
			);

			THEN("The vertices are visited in a valid order")
			{
				REQUIRE(VisitOrder.Contains(MakeTuple(0, 1)));
				REQUIRE(VisitOrder.Contains(MakeTuple(0, 2)));
				REQUIRE(VisitOrder.Contains(MakeTuple(0, 6)));
				REQUIRE(VisitOrder.Contains(MakeTuple(1, 3)));
				REQUIRE(VisitOrder.Contains(MakeTuple(2, 3))); 
				REQUIRE(VisitOrder.Contains(MakeTuple(3, 6)));
				REQUIRE(!VisitOrder.Contains(MakeTuple(6, 7))); // This edge should have been skipped when we stopped at vertex 6
				REQUIRE(VisitOrder[MakeTuple(0, 1)] < VisitOrder[MakeTuple(1, 3)]);
				REQUIRE(VisitOrder[MakeTuple(0, 2)] < VisitOrder[MakeTuple(2, 3)]);
				REQUIRE(VisitOrder[MakeTuple(1, 3)] < VisitOrder[MakeTuple(3, 6)]);
				REQUIRE(VisitOrder[MakeTuple(2, 3)] < VisitOrder[MakeTuple(3, 6)]);
			}
		}

		WHEN("the tree leaves are retrieved")
		{
			FDirectedTree Tree;
			TSet<FDirectedEdge> NewEdges(Edges);
			// Add a couple more edges to test the child-less case : 
			NewEdges.Add(FDirectedEdge(6, 8)); 
			NewEdges.Add(FDirectedEdge(8, 9));
			BuildDirectedTree(NewEdges.Array(), Tree);

			// Manipulate the tree after creation to create a situation where there's a node without a child, which should still be considered as a leaf :
			// Doing this will remove the edge from 8 to 9 while keeping node #8 : 
			Tree.FindChecked(8).Children.Remove(9);
			
			TArray<int32> Leaves;
			FindLeaves(0, Tree, Leaves);

			THEN("All leaves are properly found")
			{
				REQUIRE(Leaves.Num() == 2);
				REQUIRE(Leaves.Contains(7));
				REQUIRE(Leaves.Contains(8));
			}
		}

		WHEN("the transpose tree is traversed depth first")
		{
			FDirectedTree TransposeTree;
			BuildTransposeDirectedTree(Edges.Array(), TransposeTree);

			TMap<int32, int32> VisitOrder;
			int32 VisitCounter = 0;

			DepthFirstNodeTraversal(7, TransposeTree, [&](int32 Vertex) -> bool
				{
					VisitOrder.Add(Vertex, VisitCounter);
					VisitCounter++;
					
					if (Vertex == 3)
					{
						return false;
					} 
					return true;
				}
			);

			THEN("The vertices are visited in a valid order")
			{
				REQUIRE(VisitOrder.Contains(0));
				REQUIRE(!VisitOrder.Contains(1));
				REQUIRE(!VisitOrder.Contains(2));
				REQUIRE(VisitOrder.Contains(3));
				REQUIRE(!VisitOrder.Contains(4));
				REQUIRE(!VisitOrder.Contains(5));
				REQUIRE(VisitOrder.Contains(6));
				REQUIRE(VisitOrder.Contains(7));

				int32 Order0 = VisitOrder[0];
				int32 Order3 = VisitOrder[3];
				int32 Order6 = VisitOrder[6];
				int32 Order7 = VisitOrder[7];

				REQUIRE(Order0 >= 0);
				REQUIRE(Order3 >= 0);
				REQUIRE(Order6 >= 0);
				REQUIRE(Order7 >= 0);

				REQUIRE(Order7 < Order0);
				REQUIRE(Order7 < Order3);
				REQUIRE(Order7 < Order6);
				REQUIRE(Order6 < Order0);
				REQUIRE(Order6 < Order3);
			}
		}

		WHEN("the transpose tree is traversed depth first for edges")
		{
			FDirectedTree Tree;
			BuildTransposeDirectedTree(Edges.Array(), Tree);

			TMap<TTuple<int32, int32>, int32> VisitOrder;
			int32 VisitCounter = 0;

			DepthFirstEdgeTraversal(7, Tree, [&](int32 SourceVertex, int32 DestinationVertex) -> bool
			{
				VisitOrder.Add(MakeTuple(SourceVertex, DestinationVertex), VisitCounter);
				VisitCounter++;

				// Stop at one of the edges : 
				if ((SourceVertex == 3) && (DestinationVertex == 1))
				{
					return false;
				}
				return true;
			}
			);

			THEN("The vertices are visited in a valid order")
			{
				REQUIRE(VisitOrder.Contains(MakeTuple(7, 6)));
				REQUIRE(VisitOrder.Contains(MakeTuple(6, 0)));
				REQUIRE(VisitOrder.Contains(MakeTuple(6, 3)));
				REQUIRE(VisitOrder.Contains(MakeTuple(3, 2)));
				REQUIRE(VisitOrder.Contains(MakeTuple(2, 0)));
				REQUIRE(VisitOrder.Contains(MakeTuple(3, 1)));
				REQUIRE(!VisitOrder.Contains(MakeTuple(1, 0))); // This edge should have been skipped when we stopped at (3, 1)
				REQUIRE(VisitOrder[MakeTuple(7, 6)] == 0);
				REQUIRE(VisitOrder[MakeTuple(6, 0)] < VisitOrder[MakeTuple(3, 1)]);
				REQUIRE(VisitOrder[MakeTuple(6, 0)] < VisitOrder[MakeTuple(3, 2)]);
				REQUIRE(VisitOrder[MakeTuple(6, 3)] < VisitOrder[MakeTuple(3, 1)]);
				REQUIRE(VisitOrder[MakeTuple(6, 3)] < VisitOrder[MakeTuple(3, 2)]);
				REQUIRE(VisitOrder[MakeTuple(3, 2)] < VisitOrder[MakeTuple(2, 0)]);
			}
		}

		WHEN("the transpose tree is traversed breadth first")
		{
			FDirectedTree TransposeTree;
			BuildTransposeDirectedTree(Edges.Array(), TransposeTree);

			TMap<int32, int32> VisitOrder;
			int32 VisitCounter = 0;

			BreadthFirstNodeTraversal(7, TransposeTree, [&](int32 Vertex) -> bool
				{
					VisitOrder.Add(Vertex, VisitCounter);
					VisitCounter++;
					
					if (Vertex == 3)
					{
						return false;
					} 
					return true;
				}
			);

			THEN("The vertices are visited in a valid order")
			{
				REQUIRE(VisitOrder.Contains(0));
				REQUIRE(!VisitOrder.Contains(1));
				REQUIRE(!VisitOrder.Contains(2));
				REQUIRE(VisitOrder.Contains(3));
				REQUIRE(!VisitOrder.Contains(4));
				REQUIRE(!VisitOrder.Contains(5));
				REQUIRE(VisitOrder.Contains(6));
				REQUIRE(VisitOrder.Contains(7));

				int32 Order0 = VisitOrder[0];
				int32 Order3 = VisitOrder[3];
				int32 Order6 = VisitOrder[6];
				int32 Order7 = VisitOrder[7];

				REQUIRE(Order0 >= 0);
				REQUIRE(Order3 >= 0);
				REQUIRE(Order6 >= 0);
				REQUIRE(Order7 >= 0);

				REQUIRE(Order7 < Order0);
				REQUIRE(Order7 < Order3);
				REQUIRE(Order7 < Order6);
				REQUIRE(Order6 < Order0);
				REQUIRE(Order6 < Order3);
			}

		}

		WHEN("the transpose tree is traversed breadth first for edges")
		{
			FDirectedTree Tree;
			BuildTransposeDirectedTree(Edges.Array(), Tree);

			TMap<TTuple<int32, int32>, int32> VisitOrder;
			int32 VisitCounter = 0;

			BreadthFirstEdgeTraversal(7, Tree, [&](int32 SourceVertex, int32 DestinationVertex) -> bool
			{
				VisitOrder.Add(MakeTuple(SourceVertex, DestinationVertex), VisitCounter);
				VisitCounter++;

				// Stop at one of the edges : 
				if ((SourceVertex == 3) && (DestinationVertex == 1))
				{
					return false;
				}
				return true;
			}
			);

			THEN("The vertices are visited in a valid order")
			{
				REQUIRE(VisitOrder.Contains(MakeTuple(7, 6)));
				REQUIRE(VisitOrder.Contains(MakeTuple(6, 3)));
				REQUIRE(VisitOrder.Contains(MakeTuple(6, 0)));
				REQUIRE(VisitOrder.Contains(MakeTuple(3, 2)));
				REQUIRE(VisitOrder.Contains(MakeTuple(3, 1)));
				REQUIRE(VisitOrder.Contains(MakeTuple(2, 0)));
				REQUIRE(!VisitOrder.Contains(MakeTuple(1, 0))); // This edge should have been skipped when we stopped at (3, 1)
				REQUIRE(VisitOrder[MakeTuple(7, 6)] == 0);
				REQUIRE(VisitOrder[MakeTuple(6, 3)] < VisitOrder[MakeTuple(3, 1)]);
				REQUIRE(VisitOrder[MakeTuple(6, 0)] < VisitOrder[MakeTuple(3, 1)]);
				REQUIRE(VisitOrder[MakeTuple(6, 3)] < VisitOrder[MakeTuple(3, 2)]);
				REQUIRE(VisitOrder[MakeTuple(6, 0)] < VisitOrder[MakeTuple(3, 2)]);
				REQUIRE(VisitOrder[MakeTuple(3, 1)] < VisitOrder[MakeTuple(2, 0)]);
			}
		}
		
		WHEN("the transpose tree leaves are retrieved")
		{
			FDirectedTree Tree;
			BuildTransposeDirectedTree(Edges.Array(), Tree);

			TArray<int32> Leaves;
			FindLeaves(0, Tree, Leaves);

			THEN("All leaves are properly found")
			{
				REQUIRE(Leaves.Num() == 1);
				REQUIRE(Leaves[0] == 0);
			}
		}
	}

	GIVEN("A graph with strongly connected components")
	{
		TSet<FDirectedEdge> Edges;

		Edges.Add(FDirectedEdge(0, 1));
		Edges.Add(FDirectedEdge(0, 2));
		Edges.Add(FDirectedEdge(1, 3));
		Edges.Add(FDirectedEdge(2, 3));
		Edges.Add(FDirectedEdge(3, 6));
		Edges.Add(FDirectedEdge(6, 0));
		Edges.Add(FDirectedEdge(6, 7));
		Edges.Add(FDirectedEdge(7, 8));
		Edges.Add(FDirectedEdge(8, 9));
		Edges.Add(FDirectedEdge(9, 8));


		WHEN("the graph is analyzed")
		{
			TArray<FStronglyConnectedComponent> OutComponents;
			bool bResult = TarjanStronglyConnectedComponents(Edges, OutComponents);

			THEN("strongly connected components are found")
			{
				REQUIRE(bResult == true);
				REQUIRE(OutComponents.Num() == 2);
			}

			THEN("the strongly connected components are correct")
			{
				// Components are always reported in reverse order using the
				// tarjan algorithm. This first requirement is checking for the
				// last strongly connected component in the graph.
				FStronglyConnectedComponent& Comp = OutComponents[0];

				REQUIRE(Comp.Vertices.Num() == 2);
				REQUIRE(Comp.Vertices.Contains(8));
				REQUIRE(Comp.Vertices.Contains(9));

				REQUIRE(Comp.Edges.Num() == 2);
				REQUIRE(Comp.Edges.Contains(FDirectedEdge(8, 9)));
				REQUIRE(Comp.Edges.Contains(FDirectedEdge(9, 8)));

				Comp = OutComponents[1];

				REQUIRE(Comp.Vertices.Num() == 5);
				REQUIRE(Comp.Vertices.Contains(0));
				REQUIRE(Comp.Vertices.Contains(1));
				REQUIRE(Comp.Vertices.Contains(2));
				REQUIRE(Comp.Vertices.Contains(3));
				REQUIRE(Comp.Vertices.Contains(6));

				REQUIRE(Comp.Edges.Num() == 6);
				REQUIRE(Comp.Edges.Contains(FDirectedEdge(0, 1)));
				REQUIRE(Comp.Edges.Contains(FDirectedEdge(0, 2)));
				REQUIRE(Comp.Edges.Contains(FDirectedEdge(1, 3)));
				REQUIRE(Comp.Edges.Contains(FDirectedEdge(2, 3)));
				REQUIRE(Comp.Edges.Contains(FDirectedEdge(3, 6)));
				REQUIRE(Comp.Edges.Contains(FDirectedEdge(6, 0)));
			}
		}

		WHEN("the tree leaves are retrieved")
		{
			FDirectedTree Tree;
			BuildTransposeDirectedTree(Edges.Array(), Tree);

			TArray<int32> Leaves;
			FindLeaves(0, Tree, Leaves);

			THEN("No leaves are found")
			{
				REQUIRE(Leaves.IsEmpty());
			}
		}
	}
}

//#endif // WITH_LOW_LEVEL_TESTS