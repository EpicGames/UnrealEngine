// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/catch_test_macros.hpp>

#include "Math/SpringMath.h"

// ---------------------------------------------------------------------------
// Helper functions (migrated from EngineTest/AnimationTests/SpringDamperTests.cpp)
// ---------------------------------------------------------------------------

namespace UE::SpringMathTests
{

inline bool ApproxEquals(double A, double B, double Threshold)
{
	return FMath::Abs(A - B) < Threshold;
}

inline float SmoothingTimeToDamping(float SmoothingTime)
{
	return 4.0f / FMath::Max(SmoothingTime, UE_SMALL_NUMBER);
}

inline void ExactCriticalSpringDamper(double& InOutX, double& InOutV, double TargetX, float SmoothingTime, float DeltaTime)
{
	float Y = SmoothingTimeToDamping(SmoothingTime) / 2.0f;
	double J0 = InOutX - TargetX;
	double J1 = InOutV + J0 * Y;
	float EyDt = FMath::Exp(-Y * DeltaTime);

	InOutX = EyDt * (J0 + J1 * DeltaTime) + TargetX;
	InOutV = EyDt * (InOutV - J1 * Y * DeltaTime);
}

inline double CalculateCharacterStoppingTime(double InitialVelocity, double InitialAcceleration, float SmoothingTime, float Threshold, float TotalTime, float DeltaTime)
{
	double Time = 0.0f;
	double BelowTime = 0.0f;
	bool bIsBelowThreshold = FMath::Abs(InitialVelocity) < Threshold;
	while (Time < TotalTime)
	{
		if (bIsBelowThreshold && FMath::Abs(InitialVelocity) > Threshold)
		{
			bIsBelowThreshold = false;
		}

		if (!bIsBelowThreshold && FMath::Abs(InitialVelocity) < Threshold)
		{
			bIsBelowThreshold = true;
			BelowTime = Time;
		}

		ExactCriticalSpringDamper(InitialVelocity, InitialAcceleration, 0.0, SmoothingTime, DeltaTime);
		Time += DeltaTime;
	}
	return BelowTime;
}

inline double CalculateCharacterStoppingDistance(double InitialVelocity, double InitialAcceleration, float SmoothingTime, float TotalTime, float DeltaTime)
{
	double Time = 0.0f;
	double Distance = 0.0f;
	while (Time < TotalTime)
	{
		ExactCriticalSpringDamper(InitialVelocity, InitialAcceleration, 0.0, SmoothingTime, DeltaTime);
		Distance += FMath::Abs(InitialVelocity) * DeltaTime;
		Time += DeltaTime;
	}
	return Distance;
}

inline double CalculateCharacterStartingTime(double TargetVelocity, float SmoothingTime, float Threshold, float DeltaTime, const int32 MaxIterations = 100000)
{
	double Time = 0.0f;
	double Velocity = 0.0f;
	double Acceleration = 0.0f;
	for (int32 Iteration = 0; Iteration < MaxIterations; Iteration++)
	{
		if (FMath::Abs(Velocity - TargetVelocity) < Threshold)
		{
			break;
		}
		ExactCriticalSpringDamper(Velocity, Acceleration, TargetVelocity, SmoothingTime, DeltaTime);
		Time += DeltaTime;
	}
	return Time;
}

inline double CalculateCharacterStartingDistance(double TargetVelocity, float SmoothingTime, float Threshold, float DeltaTime, const int32 MaxIterations = 100000)
{
	double Distance = 0.0f;
	double Velocity = 0.0f;
	double Acceleration = 0.0f;
	for (int32 Iteration = 0; Iteration < MaxIterations; Iteration++)
	{
		if (FMath::Abs(Velocity - TargetVelocity) < Threshold)
		{
			break;
		}
		ExactCriticalSpringDamper(Velocity, Acceleration, TargetVelocity, SmoothingTime, DeltaTime);
		Distance += FMath::Abs(Velocity) * DeltaTime;
	}
	return Distance;
}

inline double CalculateCharacterMaximumAcceleration(double InitialVelocity, double InitialAcceleration, double TargetVelocity, float SmoothingTime, float TotalTime, int32 Steps)
{
	double Maximum = FMath::Abs(InitialAcceleration);
	for (int32 Idx = 0; Idx < Steps; Idx++)
	{
		ExactCriticalSpringDamper(InitialVelocity, InitialAcceleration, TargetVelocity, SmoothingTime, TotalTime / Steps);
		Maximum = FMath::Max(FMath::Abs(InitialAcceleration), Maximum);
	}
	return Maximum;
}

} // namespace UE::SpringMathTests

using namespace UE::SpringMathTests;

// ---------------------------------------------------------------------------
// CriticalSpringDamper exact analytical verification
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::CriticalSpringDamper::ExactAnalyticalComparison", "[SpringMath][unit][MustPass]")
{
	const double SmoothingTime = 1.0f;
	const double DeltaTime = 0.01f;
	const float Time = SmoothingTime * 2.0;
	const double X0 = 1.0;
	const double V0 = 1.0;
	const double Target = 0.0f;

	double Value = X0;
	double ValueRate = V0;

	const int32 NumSteps = static_cast<int32>(Time / DeltaTime);
	for (int32 Step = 0; Step < NumSteps; ++Step)
	{
		SpringMath::CriticalSpringDamper(Value, ValueRate, Target, SmoothingTime, DeltaTime);
	}

	// Compute expected state using exact analytical solution
	const double W0 = 2.0 / SmoothingTime;
	const double E = FMath::Exp(-W0 * Time);
	const double ExpectedX = (X0 + (V0 + W0 * X0) * Time) * E;
	const double ExpectedV = (V0 - W0 * (V0 + W0 * X0) * Time) * E;

	const double ValueTol = FMath::Max(FMath::Abs(ExpectedX) * 0.05, (double)KINDA_SMALL_NUMBER);
	const double ValueRateTol = FMath::Max(FMath::Abs(ExpectedV) * 0.05, (double)KINDA_SMALL_NUMBER);

	REQUIRE(ApproxEquals(Value, ExpectedX, ValueTol));
	REQUIRE(ApproxEquals(ValueRate, ExpectedV, ValueRateTol));
}

// ---------------------------------------------------------------------------
// CharacterStopping — parametric sweep
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::CharacterStopping::StoppingTimeAgreesWithSimulation", "[SpringMath][unit][MustPass]")
{
	const int32 VelNum = 31;
	const float SmoothingTime = 0.5213f;

	for (float VelocityThreshold : { 0.01f, 0.1f, 1.0f })
	{
		for (int32 VelIdx = 0; VelIdx < VelNum; VelIdx++)
		{
			const float Vel = 20.0f * ((float)VelIdx / (VelNum - 1) - 0.5f);

			const float StoppingTime = SpringMath::SpringCharacterStoppingTime(Vel, SmoothingTime, VelocityThreshold);
			const double StoppingTimeIntegral = CalculateCharacterStoppingTime(Vel, 0.0f, SmoothingTime, VelocityThreshold, 10.0f, 1.0f / 1000.0f);

			REQUIRE(ApproxEquals(StoppingTime, StoppingTimeIntegral, 0.05f));

			// Verify: at the stopping time, velocity should be at the threshold
			if (FMath::Abs(Vel) > VelocityThreshold)
			{
				double StoppingTimeVelocity = Vel, StoppingTimeAcc = 0.0;
				ExactCriticalSpringDamper(StoppingTimeVelocity, StoppingTimeAcc, 0.0, SmoothingTime, StoppingTime);
				REQUIRE(ApproxEquals(FMath::Abs(StoppingTimeVelocity), VelocityThreshold, 0.01f));
			}

			// Roundtrip: recover smoothing time from stopping time
			if (FMath::Abs(Vel) > UE_SMALL_NUMBER && FMath::Abs(StoppingTime) > UE_SMALL_NUMBER)
			{
				const float TimeFittedSmoothingTime = SpringMath::SpringCharacterSmoothingTimeFromStoppingTime(Vel, StoppingTime, VelocityThreshold);
				REQUIRE(ApproxEquals(TimeFittedSmoothingTime, SmoothingTime, 0.01f));
			}
		}
	}
}

TEST_CASE("SpringMath::CharacterStopping::StoppingDistanceAgreesWithSimulation", "[SpringMath][unit][MustPass]")
{
	const int32 VelNum = 31;
	const int32 AccNum = 31;
	const float SmoothingTime = 0.5213f;

	for (int32 VelIdx = 0; VelIdx < VelNum; VelIdx++)
	{
		const float Vel = 20.0f * ((float)VelIdx / (VelNum - 1) - 0.5f);

		for (int32 AccIdx = 0; AccIdx < AccNum; AccIdx++)
		{
			const float Acc = 20.0f * ((float)AccIdx / (AccNum - 1) - 0.5f);

			const float StoppingDist = SpringMath::SpringCharacterStoppingDistance(Vel, Acc, SmoothingTime);
			const double StoppingDistIntegral = CalculateCharacterStoppingDistance(Vel, Acc, SmoothingTime, 10.0f, 1.0f / 1000.0f);

			REQUIRE(ApproxEquals(StoppingDist, StoppingDistIntegral, 0.01f));

			// Roundtrip: recover smoothing time from stopping distance (only valid with zero acceleration)
			if (Acc == 0.0f && FMath::Abs(Vel) > UE_SMALL_NUMBER && FMath::Abs(StoppingDist) > UE_SMALL_NUMBER)
			{
				const float DistFittedSmoothingTime = SpringMath::SpringCharacterSmoothingTimeFromStoppingDistance(Vel, StoppingDist);
				REQUIRE(ApproxEquals(DistFittedSmoothingTime, SmoothingTime, 0.01f));
			}
		}
	}
}

// ---------------------------------------------------------------------------
// CharacterStarting — parametric sweep
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::CharacterStarting::AgreesWithSimulation", "[SpringMath][unit][MustPass]")
{
	const int32 VelNum = 31;
	const float SmoothingTime = 0.5213f;

	for (float VelocityThreshold : { 0.01f, 0.1f, 1.0f })
	{
		for (int32 VelIdx = 0; VelIdx < VelNum; VelIdx++)
		{
			const float Vel = 20.0f * ((float)VelIdx / (VelNum - 1) - 0.5f);

			const float StartingTime = SpringMath::SpringCharacterStartingTime(Vel, SmoothingTime, VelocityThreshold);
			const float StartingDist = SpringMath::SpringCharacterStartingDistance(Vel, SmoothingTime, VelocityThreshold);

			// Verify: at the starting time, velocity should be at (TargetVelocity - threshold)
			if (FMath::Abs(Vel) > VelocityThreshold)
			{
				double StartingTimeVelocity = 0.0f, StartingTimeAcc = 0.0;
				ExactCriticalSpringDamper(StartingTimeVelocity, StartingTimeAcc, Vel, SmoothingTime, StartingTime);
				REQUIRE(ApproxEquals(StartingTimeVelocity, Vel < 0.0f ? Vel + VelocityThreshold : Vel - VelocityThreshold, 0.01f));
			}

			const double StartingTimeIntegral = CalculateCharacterStartingTime(Vel, SmoothingTime, VelocityThreshold, 1.0f / 1000.0f);
			const double StartingDistIntegral = CalculateCharacterStartingDistance(Vel, SmoothingTime, VelocityThreshold, 1.0f / 1000.0f);

			REQUIRE(ApproxEquals(StartingTime, StartingTimeIntegral, 0.01f));
			REQUIRE(ApproxEquals(StartingDist, StartingDistIntegral, 0.05f));

			// Roundtrip: recover smoothing time from starting time and distance
			if (FMath::Abs(Vel) > UE_SMALL_NUMBER && FMath::Abs(StartingTime) > UE_SMALL_NUMBER && FMath::Abs(StartingDist) > UE_SMALL_NUMBER)
			{
				const float TimeFittedSmoothingTime = SpringMath::SpringCharacterSmoothingTimeFromStartingTime(Vel, StartingTime, VelocityThreshold);
				const float DistFittedSmoothingTime = SpringMath::SpringCharacterSmoothingTimeFromStartingDistance(Vel, StartingDist, VelocityThreshold);
				REQUIRE(ApproxEquals(TimeFittedSmoothingTime, SmoothingTime, 0.01f));
				REQUIRE(ApproxEquals(DistFittedSmoothingTime, SmoothingTime, 0.01f));
			}
		}
	}
}

// ---------------------------------------------------------------------------
// CharacterMaxAcceleration — parametric sweep
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::CharacterMaxAcceleration::AgreesWithSimulation", "[SpringMath][unit][MustPass]")
{
	const int32 VelNum = 31;
	const int32 AccNum = 31;
	const float SmoothingTime = 0.5213f;

	for (int32 VelIdx = 0; VelIdx < VelNum; VelIdx++)
	{
		const float Vel = 20.0f * ((float)VelIdx / (VelNum - 1) - 0.5f);

		for (int32 AccIdx = 0; AccIdx < AccNum; AccIdx++)
		{
			const float Acc = 20.0f * ((float)AccIdx / (AccNum - 1) - 0.5f);

			const float MaximumAcceleration = SpringMath::SpringCharacterMaximumAcceleration(Vel, Acc, 0.0f, SmoothingTime);
			const double MaximumAccelerationIntegral = CalculateCharacterMaximumAcceleration(Vel, Acc, 0.0f, SmoothingTime, 10.0f, 10000);

			REQUIRE(ApproxEquals(MaximumAcceleration, MaximumAccelerationIntegral, 0.01f));

			// Roundtrip: recover smoothing time from max acceleration (only valid with zero initial acceleration)
			if (Acc == 0.0f && FMath::Abs(MaximumAcceleration) > UE_SMALL_NUMBER && FMath::Abs(Vel) > UE_SMALL_NUMBER)
			{
				const float FittedSmoothingTime = SpringMath::SpringCharacterSmoothingTimeFromMaximumAcceleration(Vel, 0.0f, MaximumAcceleration);
				REQUIRE(ApproxEquals(FittedSmoothingTime, SmoothingTime, 0.01f));
			}
		}
	}
}

// ---------------------------------------------------------------------------
// Additional tests not in the old file
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::SpringCharacterMaximumAngularVelocity::ReturnsPositiveValue", "[SpringMath][unit][MustPass]")
{
	const float MaxAngVel = SpringMath::SpringCharacterMaximumAngularVelocity(0.0f, 0.0f, UE_PI / 2.0f, 0.5f);
	REQUIRE(MaxAngVel > 0.0f);
}

TEST_CASE("SpringMath::SpringCharacterStoppingInflectionTime::ZeroInputsReturnsEmpty", "[SpringMath][unit][MustPass]")
{
	const TOptional<float> Result = SpringMath::SpringCharacterStoppingInflectionTime(0.0f, 0.0f, 0.5f);
	REQUIRE(!Result.IsSet());
}

TEST_CASE("SpringMath::SpringCharacterStoppingInflectionTime::NonZeroInputsReturnsPositiveOrEmpty", "[SpringMath][unit][MustPass]")
{
	{
		const TOptional<float> Result = SpringMath::SpringCharacterStoppingInflectionTime(100.0f, 50.0f, 0.5f);
		if (Result.IsSet())
		{
			CHECK(Result.GetValue() > 0.0f);
		}
	}

	{
		const TOptional<float> Result = SpringMath::SpringCharacterStoppingInflectionTime(100.0f, -200.0f, 0.5f);
		if (Result.IsSet())
		{
			CHECK(Result.GetValue() > 0.0f);
		}
	}
}
