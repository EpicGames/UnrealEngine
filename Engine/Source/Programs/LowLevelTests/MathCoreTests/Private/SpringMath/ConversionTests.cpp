// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/catch_test_macros.hpp>

#include "Math/SpringMath.h"

TEST_CASE("SpringMath::Conversion::SmoothingTimeToHalfLife roundtrip", "[SpringMath][unit][MustPass]")
{
	constexpr float Tolerance = 1e-4f;
	const float TestValues[] = { 0.1f, 0.5f, 1.0f, 2.0f };

	for (float SmoothingTime : TestValues)
	{
		float HalfLife = SpringMath::SmoothingTimeToHalfLife(SmoothingTime);
		float Recovered = SpringMath::HalfLifeToSmoothingTime(HalfLife);
		REQUIRE(FMath::IsNearlyEqual(Recovered, SmoothingTime, Tolerance));
	}
}

TEST_CASE("SpringMath::Conversion::SmoothingTimeToHalfLife formula", "[SpringMath][unit][MustPass]")
{
	constexpr float Tolerance = 1e-4f;
	const float TestValues[] = { 0.1f, 0.5f, 1.0f, 2.0f };

	for (float SmoothingTime : TestValues)
	{
		float HalfLife = SpringMath::SmoothingTimeToHalfLife(SmoothingTime);
		float Expected = SmoothingTime * UE_LN2;
		REQUIRE(FMath::IsNearlyEqual(HalfLife, Expected, Tolerance));
	}
}

TEST_CASE("SpringMath::Conversion::HalfLifeToSmoothingTime roundtrip", "[SpringMath][unit][MustPass]")
{
	constexpr float Tolerance = 1e-4f;
	const float TestValues[] = { 0.1f, 0.5f, 1.0f, 2.0f };

	for (float HalfLife : TestValues)
	{
		float SmoothingTime = SpringMath::HalfLifeToSmoothingTime(HalfLife);
		float Recovered = SpringMath::SmoothingTimeToHalfLife(SmoothingTime);
		REQUIRE(FMath::IsNearlyEqual(Recovered, HalfLife, Tolerance));
	}
}

TEST_CASE("SpringMath::Conversion::SmoothingTimeToStrength roundtrip", "[SpringMath][unit][MustPass]")
{
	constexpr float Tolerance = 1e-4f;
	const float TestValues[] = { 0.1f, 0.5f, 1.0f, 2.0f };

	for (float SmoothingTime : TestValues)
	{
		float Strength = SpringMath::SmoothingTimeToStrength(SmoothingTime);
		float Recovered = SpringMath::StrengthToSmoothingTime(Strength);
		REQUIRE(FMath::IsNearlyEqual(Recovered, SmoothingTime, Tolerance));
	}
}

TEST_CASE("SpringMath::Conversion::SmoothingTimeToStrength formula", "[SpringMath][unit][MustPass]")
{
	constexpr float Tolerance = 1e-4f;
	const float TestValues[] = { 0.1f, 0.5f, 1.0f, 2.0f };

	for (float SmoothingTime : TestValues)
	{
		float Strength = SpringMath::SmoothingTimeToStrength(SmoothingTime);
		float Expected = 2.0f / SmoothingTime;
		REQUIRE(FMath::IsNearlyEqual(Strength, Expected, Tolerance));
	}
}

TEST_CASE("SpringMath::Conversion::StrengthToSmoothingTime roundtrip", "[SpringMath][unit][MustPass]")
{
	constexpr float Tolerance = 1e-4f;
	const float TestValues[] = { 0.1f, 0.5f, 1.0f, 2.0f };

	for (float Strength : TestValues)
	{
		float SmoothingTime = SpringMath::StrengthToSmoothingTime(Strength);
		float Recovered = SpringMath::SmoothingTimeToStrength(SmoothingTime);
		REQUIRE(FMath::IsNearlyEqual(Recovered, Strength, Tolerance));
	}
}

TEST_CASE("SpringMath::Conversion::EdgeCase very small smoothing time does not crash", "[SpringMath][unit][MustPass]")
{
	// Near UE_KINDA_SMALL_NUMBER - should not crash or produce NaN/Inf
	constexpr float SmallTime = UE_KINDA_SMALL_NUMBER * 2.0f;

	float HalfLife = SpringMath::SmoothingTimeToHalfLife(SmallTime);
	CHECK(!FMath::IsNaN(HalfLife));
	CHECK(FMath::IsFinite(HalfLife));

	float SmoothingTimeBack = SpringMath::HalfLifeToSmoothingTime(HalfLife);
	CHECK(!FMath::IsNaN(SmoothingTimeBack));
	CHECK(FMath::IsFinite(SmoothingTimeBack));

	float Strength = SpringMath::SmoothingTimeToStrength(SmallTime);
	CHECK(!FMath::IsNaN(Strength));
	CHECK(FMath::IsFinite(Strength));

	float SmoothingTimeFromStrength = SpringMath::StrengthToSmoothingTime(Strength);
	CHECK(!FMath::IsNaN(SmoothingTimeFromStrength));
	CHECK(FMath::IsFinite(SmoothingTimeFromStrength));
}
