// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/catch_test_macros.hpp>

#include "Math/SpringMath.h"

//---------------------------------------------------------------------
// CriticalSpringDamper<float>
//---------------------------------------------------------------------

TEST_CASE("SpringMath::CriticalSpringDamper::Float convergence to target", "[SpringMath][unit][MustPass]")
{
	float X = 10.0f;
	float V = 0.0f;
	constexpr float Target = 0.0f;
	constexpr float SmoothingTime = 0.5f;
	constexpr float DeltaTime = 1.0f / 60.0f;
	constexpr int32 Steps = 1000;

	for (int32 Step = 0; Step < Steps; ++Step)
	{
		SpringMath::CriticalSpringDamper(X, V, Target, SmoothingTime, DeltaTime);
	}

	REQUIRE(FMath::IsNearlyEqual(X, Target, 1e-3f));
	REQUIRE(FMath::IsNearlyEqual(V, 0.0f, 1e-3f));
}

TEST_CASE("SpringMath::CriticalSpringDamper::Float SmoothingTime zero snaps to target", "[SpringMath][unit][MustPass]")
{
	float X = 10.0f;
	float V = 0.0f;
	constexpr float Target = 5.0f;
	constexpr float SmoothingTime = 0.0f;
	constexpr float DeltaTime = 1.0f / 60.0f;

	SpringMath::CriticalSpringDamper(X, V, Target, SmoothingTime, DeltaTime);

	// SmoothingTime=0 snaps exactly to target
	REQUIRE(X == Target);
	REQUIRE(V == 0.0f);
}

TEST_CASE("SpringMath::CriticalSpringDamper::Float monotonic approach (no oscillation)", "[SpringMath][unit][MustPass]")
{
	float X = 10.0f;
	float V = 0.0f;
	constexpr float Target = 0.0f;
	constexpr float SmoothingTime = 0.5f;
	constexpr float DeltaTime = 1.0f / 60.0f;
	constexpr int32 Steps = 200;

	float PreviousDistance = FMath::Abs(X - Target);

	for (int32 Step = 0; Step < Steps; ++Step)
	{
		SpringMath::CriticalSpringDamper(X, V, Target, SmoothingTime, DeltaTime);
		float CurrentDistance = FMath::Abs(X - Target);
		// Each step should bring us closer or keep us at the same distance (critically damped, no oscillation)
		CHECK(CurrentDistance <= PreviousDistance + 1e-6f);
		PreviousDistance = CurrentDistance;
	}
}

//---------------------------------------------------------------------
// CriticalSpringDamper<FVector>
//---------------------------------------------------------------------

TEST_CASE("SpringMath::CriticalSpringDamper::FVector convergence to target", "[SpringMath][unit][MustPass]")
{
	FVector X(10.0f, 20.0f, 30.0f);
	FVector V = FVector::ZeroVector;
	FVector Target = FVector::ZeroVector;
	constexpr float SmoothingTime = 0.5f;
	constexpr float DeltaTime = 1.0f / 60.0f;
	constexpr int32 Steps = 1000;

	for (int32 Step = 0; Step < Steps; ++Step)
	{
		SpringMath::CriticalSpringDamper(X, V, Target, SmoothingTime, DeltaTime);
	}

	REQUIRE(X.Equals(Target, 1e-3f));
	REQUIRE(V.Equals(FVector::ZeroVector, 1e-3f));
}

//---------------------------------------------------------------------
// CriticalSpringDamperAngle
//---------------------------------------------------------------------

TEST_CASE("SpringMath::CriticalSpringDamperAngle::Wrapping takes shortest path", "[SpringMath][unit][MustPass]")
{
	// Start at 350 degrees, target 10 degrees. Shortest path is 20 degrees forward, not 340 backward.
	float AngleRadians = FMath::DegreesToRadians(350.0f);
	float AngularVelocity = 0.0f;
	const float TargetRadians = FMath::DegreesToRadians(10.0f);
	constexpr float SmoothingTime = 0.5f;
	constexpr float DeltaTime = 1.0f / 60.0f;

	// After one step, verify we moved toward the target via the short path (angle should increase past 360/wrap)
	float InitialAngle = AngleRadians;
	SpringMath::CriticalSpringDamperAngle(AngleRadians, AngularVelocity, TargetRadians, SmoothingTime, DeltaTime);

	// The delta from 350 to 10 via shortest path is +20 degrees. The spring should move the angle
	// in the positive direction (wrapping through 360), so the angular velocity should be positive.
	CHECK(AngularVelocity > 0.0f);

	// Run many more steps to verify convergence
	constexpr int32 Steps = 1000;
	for (int32 Step = 0; Step < Steps; ++Step)
	{
		SpringMath::CriticalSpringDamperAngle(AngleRadians, AngularVelocity, TargetRadians, SmoothingTime, DeltaTime);
	}

	// After convergence, the angle difference to target should be near zero
	float DeltaAngle = FMath::FindDeltaAngleRadians(TargetRadians, AngleRadians);
	REQUIRE(FMath::IsNearlyEqual(DeltaAngle, 0.0f, 1e-3f));
}

//---------------------------------------------------------------------
// CriticalSpringDamperQuat
//---------------------------------------------------------------------

TEST_CASE("SpringMath::CriticalSpringDamperQuat::Convergence to target rotation", "[SpringMath][unit][MustPass]")
{
	FQuat Rotation = FQuat::Identity;
	FVector AngularVelocity = FVector::ZeroVector;
	const FQuat TargetRotation = FQuat(FVector::UpVector, UE_PI / 2.0f);
	constexpr float SmoothingTime = 0.5f;
	constexpr float DeltaTime = 1.0f / 60.0f;
	constexpr int32 Steps = 1000;

	for (int32 Step = 0; Step < Steps; ++Step)
	{
		SpringMath::CriticalSpringDamperQuat(Rotation, AngularVelocity, TargetRotation, SmoothingTime, DeltaTime);
	}

	REQUIRE(Rotation.Equals(TargetRotation, 1e-3f));
	REQUIRE(AngularVelocity.Equals(FVector::ZeroVector, 1e-3f));
}

//---------------------------------------------------------------------
// CriticalSpringDamperScale
//---------------------------------------------------------------------

TEST_CASE("SpringMath::CriticalSpringDamperScale::Convergence to target scale", "[SpringMath][unit][MustPass]")
{
	FVector Scale(2.0f, 2.0f, 2.0f);
	FVector ScalarVelocity = FVector::ZeroVector;
	const FVector TargetScale(1.0f, 1.0f, 1.0f);
	constexpr float SmoothingTime = 0.5f;
	constexpr float DeltaTime = 1.0f / 60.0f;
	constexpr int32 Steps = 1000;

	for (int32 Step = 0; Step < Steps; ++Step)
	{
		SpringMath::CriticalSpringDamperScale(Scale, ScalarVelocity, TargetScale, SmoothingTime, DeltaTime);
	}

	REQUIRE(Scale.Equals(TargetScale, 1e-3f));
}

//---------------------------------------------------------------------
// CriticalDecay<float>
//---------------------------------------------------------------------

TEST_CASE("SpringMath::CriticalDecay::Float decays to zero", "[SpringMath][unit][MustPass]")
{
	float X = 10.0f;
	float V = 5.0f;
	constexpr float SmoothingTime = 0.5f;
	constexpr float DeltaTime = 1.0f / 60.0f;
	constexpr int32 Steps = 1000;

	for (int32 Step = 0; Step < Steps; ++Step)
	{
		SpringMath::CriticalDecay(X, V, SmoothingTime, DeltaTime);
	}

	REQUIRE(FMath::IsNearlyEqual(X, 0.0f, 1e-3f));
	REQUIRE(FMath::IsNearlyEqual(V, 0.0f, 1e-3f));
}

//---------------------------------------------------------------------
// CriticalDecayAngle
//---------------------------------------------------------------------

TEST_CASE("SpringMath::CriticalDecayAngle::Angle decays to zero", "[SpringMath][unit][MustPass]")
{
	float AngleRadians = FMath::DegreesToRadians(90.0f);
	float AngularVelocity = 0.0f;
	constexpr float SmoothingTime = 0.5f;
	constexpr float DeltaTime = 1.0f / 60.0f;
	constexpr int32 Steps = 1000;

	for (int32 Step = 0; Step < Steps; ++Step)
	{
		SpringMath::CriticalDecayAngle(AngleRadians, AngularVelocity, SmoothingTime, DeltaTime);
	}

	REQUIRE(FMath::IsNearlyEqual(AngleRadians, 0.0f, 1e-3f));
	REQUIRE(FMath::IsNearlyEqual(AngularVelocity, 0.0f, 1e-3f));
}

//---------------------------------------------------------------------
// CriticalDecayQuat
//---------------------------------------------------------------------

TEST_CASE("SpringMath::CriticalDecayQuat::Rotation decays to identity", "[SpringMath][unit][MustPass]")
{
	FQuat Rotation = FQuat(FVector::UpVector, UE_PI / 4.0f);
	FVector AngularVelocity = FVector::ZeroVector;
	constexpr float SmoothingTime = 0.5f;
	constexpr float DeltaTime = 1.0f / 60.0f;
	constexpr int32 Steps = 1000;

	for (int32 Step = 0; Step < Steps; ++Step)
	{
		SpringMath::CriticalDecayQuat(Rotation, AngularVelocity, SmoothingTime, DeltaTime);
	}

	REQUIRE(Rotation.Equals(FQuat::Identity, 1e-3f));
	REQUIRE(AngularVelocity.Equals(FVector::ZeroVector, 1e-3f));
}

//---------------------------------------------------------------------
// CriticalDecayScale
//---------------------------------------------------------------------

TEST_CASE("SpringMath::CriticalDecayScale::Scale decays to uniform one", "[SpringMath][unit][MustPass]")
{
	// CriticalDecayScale works in log space: it decays toward exp(0) = 1.
	// Starting at (2,3,4), the log values decay to 0, so scale approaches (1,1,1).
	FVector Scale(2.0f, 3.0f, 4.0f);
	FVector ScalarVelocity = FVector::ZeroVector;
	constexpr float SmoothingTime = 0.5f;
	constexpr float DeltaTime = 1.0f / 60.0f;
	constexpr int32 Steps = 1000;

	for (int32 Step = 0; Step < Steps; ++Step)
	{
		SpringMath::CriticalDecayScale(Scale, ScalarVelocity, SmoothingTime, DeltaTime);
	}

	const FVector ExpectedScale(1.0f, 1.0f, 1.0f);
	REQUIRE(Scale.Equals(ExpectedScale, 1e-3f));
	REQUIRE(ScalarVelocity.Equals(FVector::ZeroVector, 1e-3f));
}
