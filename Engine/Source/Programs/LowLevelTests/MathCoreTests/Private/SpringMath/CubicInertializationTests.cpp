// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"
#include <catch2/catch_test_macros.hpp>
#include "Math/SpringMath.h"

// ---------------------------------------------------------------------------
// CubicDecayWeights
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::CubicDecayWeights::AtTimeZero", "[SpringMath][unit][MustPass]")
{
	float W0 = 0.0f, W1 = 0.0f, W2 = 0.0f, W3 = 0.0f;
	SpringMath::CubicDecayWeights(W0, W1, W2, W3, 0.0f, 1.0f);

	REQUIRE(FMath::IsNearlyEqual(W0, 1.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(W1, 0.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(W2, 0.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(W3, 1.0f, 1e-4f));
}

TEST_CASE("SpringMath::CubicDecayWeights::AtBlendTime", "[SpringMath][unit][MustPass]")
{
	float W0 = 0.0f, W1 = 0.0f, W2 = 0.0f, W3 = 0.0f;
	const float BlendTime = 1.0f;
	SpringMath::CubicDecayWeights(W0, W1, W2, W3, BlendTime, BlendTime);

	REQUIRE(FMath::IsNearlyEqual(W0, 0.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(W1, 0.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(W2, 0.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(W3, 0.0f, 1e-4f));
}

TEST_CASE("SpringMath::CubicDecayWeights::AtHalfBlendTime", "[SpringMath][unit][MustPass]")
{
	float W0 = 0.0f, W1 = 0.0f, W2 = 0.0f, W3 = 0.0f;
	const float BlendTime = 1.0f;
	SpringMath::CubicDecayWeights(W0, W1, W2, W3, BlendTime * 0.5f, BlendTime);

	// At T=0.5: intermediate weights should all be between 0 and 1 (or -1 for W2 which can be negative)
	CHECK(W0 >= 0.0f);
	CHECK(W0 <= 1.0f);
	CHECK(W1 >= 0.0f);
	CHECK(W1 <= 1.0f);
	// W2 is the velocity weight derived from the derivative, can be negative
	CHECK(W2 >= -6.0f);
	CHECK(W2 <= 6.0f);
	CHECK(W3 >= -1.0f);
	CHECK(W3 <= 1.0f);
}

// ---------------------------------------------------------------------------
// CubicInertializeApply<float>
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::CubicInertializeApply::FullOffsetAtTimeZero", "[SpringMath][unit][MustPass]")
{
	float Value = 10.0f;
	float Velocity = 0.0f;
	const float ValueOffset = 5.0f;
	const float VelocityOffset = 0.0f;

	SpringMath::CubicInertializeApply(Value, Velocity, ValueOffset, VelocityOffset, 0.0f, 1.0f);

	// At time 0 the full offset should be applied: 10 + 5*W0(=1) + 0*W1(=0) = 15
	REQUIRE(FMath::IsNearlyEqual(Value, 15.0f, 1e-4f));
}

TEST_CASE("SpringMath::CubicInertializeApply::ZeroOffsetAfterBlendTime", "[SpringMath][unit][MustPass]")
{
	float Value = 10.0f;
	float Velocity = 0.0f;
	const float ValueOffset = 5.0f;
	const float VelocityOffset = 0.0f;
	const float BlendTime = 1.0f;

	SpringMath::CubicInertializeApply(Value, Velocity, ValueOffset, VelocityOffset, BlendTime, BlendTime);

	// At BlendTime all weights are zero, so value should be unchanged
	REQUIRE(FMath::IsNearlyEqual(Value, 10.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Velocity, 0.0f, 1e-4f));
}

TEST_CASE("SpringMath::CubicInertializeApply::ZeroOffsetPastBlendTime", "[SpringMath][unit][MustPass]")
{
	float Value = 10.0f;
	float Velocity = 0.0f;
	const float ValueOffset = 5.0f;
	const float VelocityOffset = 0.0f;
	const float BlendTime = 1.0f;

	// Well past the blend time
	SpringMath::CubicInertializeApply(Value, Velocity, ValueOffset, VelocityOffset, 2.0f, BlendTime);

	// Time is clamped to [0,1] in CubicDecayWeights, so all weights are zero
	REQUIRE(FMath::IsNearlyEqual(Value, 10.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Velocity, 0.0f, 1e-4f));
}

// ---------------------------------------------------------------------------
// CubicInertializeTransition<float>
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::CubicInertializeTransition::ComputesNewOffsets", "[SpringMath][unit][MustPass]")
{
	float ValueOffset = 0.0f;
	float VelocityOffset = 0.0f;
	const float SrcValue = 20.0f;
	const float SrcVelocity = 0.0f;
	const float DstValue = 10.0f;
	const float DstVelocity = 0.0f;

	SpringMath::CubicInertializeTransition(ValueOffset, VelocityOffset, SrcValue, SrcVelocity, DstValue, DstVelocity, 0.0f, 1.0f);

	// At time 0 with zero prior offsets: Value = Src + W0*0 + W1*0 = 20
	// Offset = Value - Dst = 20 - 10 = 10
	REQUIRE(FMath::IsNearlyEqual(ValueOffset, 10.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(VelocityOffset, 0.0f, 1e-4f));
}

TEST_CASE("SpringMath::CubicInertializeTransition::ZeroOffsetWhenSrcEqualsDst", "[SpringMath][unit][MustPass]")
{
	float ValueOffset = 0.0f;
	float VelocityOffset = 0.0f;
	const float SrcValue = 10.0f;
	const float SrcVelocity = 5.0f;
	const float DstValue = 10.0f;
	const float DstVelocity = 5.0f;

	SpringMath::CubicInertializeTransition(ValueOffset, VelocityOffset, SrcValue, SrcVelocity, DstValue, DstVelocity, 0.0f, 1.0f);

	REQUIRE(FMath::IsNearlyEqual(ValueOffset, 0.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(VelocityOffset, 0.0f, 1e-4f));
}

// ---------------------------------------------------------------------------
// CubicInertializeApplyQuat
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::CubicInertializeApplyQuat::FullOffsetAtTimeZero", "[SpringMath][unit][MustPass]")
{
	FQuat Rotation = FQuat::Identity;
	FVector AngularVelocity = FVector::ZeroVector;
	// 90-degree rotation around Z
	const FQuat RotationOffset = FQuat(FVector::UpVector, FMath::DegreesToRadians(90.0f));
	const FVector AngularVelocityOffset = FVector::ZeroVector;

	SpringMath::CubicInertializeApplyQuat(Rotation, AngularVelocity, RotationOffset, AngularVelocityOffset, 0.0f, 1.0f);

	// At time 0, W0 = 1, so the full rotation offset should be applied
	CHECK(!Rotation.Equals(FQuat::Identity, 1e-4f));
}

TEST_CASE("SpringMath::CubicInertializeApplyQuat::NoOffsetAfterBlendTime", "[SpringMath][unit][MustPass]")
{
	FQuat Rotation = FQuat::Identity;
	FVector AngularVelocity = FVector::ZeroVector;
	const FQuat RotationOffset = FQuat(FVector::UpVector, FMath::DegreesToRadians(90.0f));
	const FVector AngularVelocityOffset = FVector::ZeroVector;

	SpringMath::CubicInertializeApplyQuat(Rotation, AngularVelocity, RotationOffset, AngularVelocityOffset, 1.0f, 1.0f);

	// At BlendTime the rotation should be unmodified (identity)
	REQUIRE(Rotation.Equals(FQuat::Identity, 1e-4f));
	REQUIRE(AngularVelocity.Equals(FVector::ZeroVector, 1e-4f));
}

// ---------------------------------------------------------------------------
// CubicInertializeTransitionQuat
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::CubicInertializeTransitionQuat::ComputesRotationOffset", "[SpringMath][unit][MustPass]")
{
	FQuat RotationOffset = FQuat::Identity;
	FVector AngularVelocityOffset = FVector::ZeroVector;
	const FQuat SrcRotation = FQuat(FVector::UpVector, FMath::DegreesToRadians(45.0f));
	const FVector SrcAngularVelocity = FVector::ZeroVector;
	const FQuat DstRotation = FQuat::Identity;
	const FVector DstAngularVelocity = FVector::ZeroVector;

	SpringMath::CubicInertializeTransitionQuat(
		RotationOffset, AngularVelocityOffset,
		SrcRotation, SrcAngularVelocity,
		DstRotation, DstAngularVelocity,
		0.0f, 1.0f);

	// The rotation offset should capture the difference between Src and Dst
	CHECK(!RotationOffset.Equals(FQuat::Identity, 1e-4f));
}

// ---------------------------------------------------------------------------
// CubicInertializeApplyAngle
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::CubicInertializeApplyAngle::FullOffsetAtTimeZero", "[SpringMath][unit][MustPass]")
{
	float Angle = 0.0f;
	float AngularVelocity = 0.0f;
	const float AngleOffset = FMath::DegreesToRadians(45.0f);
	const float AngularVelocityOffset = 0.0f;

	SpringMath::CubicInertializeApplyAngle(Angle, AngularVelocity, AngleOffset, AngularVelocityOffset, 0.0f, 1.0f);

	// At time 0: full offset applied
	REQUIRE(FMath::IsNearlyEqual(Angle, AngleOffset, 1e-4f));
}

TEST_CASE("SpringMath::CubicInertializeApplyAngle::NoOffsetAfterBlendTime", "[SpringMath][unit][MustPass]")
{
	float Angle = 0.0f;
	float AngularVelocity = 0.0f;
	const float AngleOffset = FMath::DegreesToRadians(45.0f);
	const float AngularVelocityOffset = 0.0f;

	SpringMath::CubicInertializeApplyAngle(Angle, AngularVelocity, AngleOffset, AngularVelocityOffset, 1.0f, 1.0f);

	REQUIRE(FMath::IsNearlyEqual(Angle, 0.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(AngularVelocity, 0.0f, 1e-4f));
}

// ---------------------------------------------------------------------------
// CubicInertializeTransitionAngle
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::CubicInertializeTransitionAngle::ComputesAngleOffset", "[SpringMath][unit][MustPass]")
{
	float AngleOffset = 0.0f;
	float AngularVelocityOffset = 0.0f;
	const float SrcAngle = FMath::DegreesToRadians(90.0f);
	const float SrcAngularVelocity = 0.0f;
	const float DstAngle = 0.0f;
	const float DstAngularVelocity = 0.0f;

	SpringMath::CubicInertializeTransitionAngle(
		AngleOffset, AngularVelocityOffset,
		SrcAngle, SrcAngularVelocity,
		DstAngle, DstAngularVelocity,
		0.0f, 1.0f);

	// The angle offset should be the delta between Src and Dst wrapped to shortest arc
	REQUIRE(FMath::IsNearlyEqual(AngleOffset, FMath::DegreesToRadians(90.0f), 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(AngularVelocityOffset, 0.0f, 1e-4f));
}

// ---------------------------------------------------------------------------
// CubicInertializeApplyScale
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::CubicInertializeApplyScale::FullOffsetAtTimeZero", "[SpringMath][unit][MustPass]")
{
	FVector Scale(1.0f, 1.0f, 1.0f);
	FVector ScalarVelocity(0.0f, 0.0f, 0.0f);
	const FVector ScaleOffset(2.0f, 2.0f, 2.0f);
	const FVector ScalarVelocityOffset(0.0f, 0.0f, 0.0f);

	SpringMath::CubicInertializeApplyScale(Scale, ScalarVelocity, ScaleOffset, ScalarVelocityOffset, 0.0f, 1.0f);

	// At time 0, W0=1 so the full scale offset should be applied in multiplicative space
	// Result = ScaleExp(1.0 * ScaleLog(2,2,2) + 0) * (1,1,1) = (2,2,2)
	REQUIRE(FMath::IsNearlyEqual(Scale.X, 2.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Scale.Y, 2.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Scale.Z, 2.0f, 1e-4f));
}

TEST_CASE("SpringMath::CubicInertializeApplyScale::NoOffsetAfterBlendTime", "[SpringMath][unit][MustPass]")
{
	FVector Scale(1.0f, 1.0f, 1.0f);
	FVector ScalarVelocity(0.0f, 0.0f, 0.0f);
	const FVector ScaleOffset(2.0f, 2.0f, 2.0f);
	const FVector ScalarVelocityOffset(0.0f, 0.0f, 0.0f);

	SpringMath::CubicInertializeApplyScale(Scale, ScalarVelocity, ScaleOffset, ScalarVelocityOffset, 1.0f, 1.0f);

	// At BlendTime all weights are zero: ScaleExp(0) = (1,1,1), so Scale * (1,1,1) = (1,1,1)
	REQUIRE(FMath::IsNearlyEqual(Scale.X, 1.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Scale.Y, 1.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Scale.Z, 1.0f, 1e-4f));
}

// ---------------------------------------------------------------------------
// CubicInertializeTransitionScale
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::CubicInertializeTransitionScale::ComputesScaleOffset", "[SpringMath][unit][MustPass]")
{
	FVector ScaleOffset(1.0f, 1.0f, 1.0f);
	FVector ScalarVelocityOffset(0.0f, 0.0f, 0.0f);
	const FVector SrcScale(2.0f, 2.0f, 2.0f);
	const FVector SrcScalarVelocity(0.0f, 0.0f, 0.0f);
	const FVector DstScale(1.0f, 1.0f, 1.0f);
	const FVector DstScalarVelocity(0.0f, 0.0f, 0.0f);

	SpringMath::CubicInertializeTransitionScale(
		ScaleOffset, ScalarVelocityOffset,
		SrcScale, SrcScalarVelocity,
		DstScale, DstScalarVelocity,
		0.0f, 1.0f);

	// At time 0 with identity prior offset: Value = ScaleExp(W0*log(1) + W1*0) * Src = (1)*Src = (2,2,2)
	// ScaleOffset = Value / Dst = (2,2,2) / (1,1,1) = (2,2,2)
	REQUIRE(FMath::IsNearlyEqual(ScaleOffset.X, 2.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(ScaleOffset.Y, 2.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(ScaleOffset.Z, 2.0f, 1e-4f));
}
