// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"
#include <catch2/catch_test_macros.hpp>
#include "Math/SpringMath.h"

// ---------------------------------------------------------------------------
// DeadBlendExtrapolate<float>
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::DeadBlendExtrapolate::ValueMovesForward", "[SpringMath][unit][MustPass]")
{
	float Value = 0.0f;
	float Velocity = 10.0f;
	const float SmoothingTime = 0.5f;

	SpringMath::DeadBlendExtrapolate(Value, Velocity, 1.0f, SmoothingTime);

	// Value should have moved forward from zero
	REQUIRE(Value > 0.0f);
	// Velocity should have decayed (less than the initial 10)
	REQUIRE(Velocity < 10.0f);
	REQUIRE(Velocity > 0.0f);
}

TEST_CASE("SpringMath::DeadBlendExtrapolate::VelocityDecays", "[SpringMath][unit][MustPass]")
{
	float Value = 5.0f;
	float Velocity = 20.0f;
	const float SmoothingTime = 0.5f;

	const float InitialVelocity = Velocity;
	SpringMath::DeadBlendExtrapolate(Value, Velocity, 1.0f, SmoothingTime);

	REQUIRE(Velocity < InitialVelocity);
	REQUIRE(Velocity > 0.0f);
}

TEST_CASE("SpringMath::DeadBlendExtrapolate::TimeZeroNoChange", "[SpringMath][unit][MustPass]")
{
	float Value = 5.0f;
	float Velocity = 10.0f;

	SpringMath::DeadBlendExtrapolate(Value, Velocity, 0.0f, 0.5f);

	// With Time=0, decay = exp(0) = 1, value += velocity * smoothing * (1-1) = 0, velocity *= 1
	REQUIRE(FMath::IsNearlyEqual(Value, 5.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Velocity, 10.0f, 1e-4f));
}

// ---------------------------------------------------------------------------
// DeadBlendExtrapolateQuat
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::DeadBlendExtrapolateQuat::RotationExtrapolatesForward", "[SpringMath][unit][MustPass]")
{
	FQuat Value = FQuat::Identity;
	FVector Velocity(0.0f, 0.0f, 2.0f); // Spinning around Z at 2 rad/s

	SpringMath::DeadBlendExtrapolateQuat(Value, Velocity, 1.0f, 0.5f);

	// Rotation should have moved away from identity
	CHECK(!Value.Equals(FQuat::Identity, 1e-4f));
	// Angular velocity should decay
	REQUIRE(Velocity.Size() < 2.0f);
	REQUIRE(Velocity.Size() > 0.0f);
}

TEST_CASE("SpringMath::DeadBlendExtrapolateQuat::TimeZeroNoChange", "[SpringMath][unit][MustPass]")
{
	FQuat Value = FQuat::Identity;
	FVector Velocity(0.0f, 0.0f, 2.0f);

	SpringMath::DeadBlendExtrapolateQuat(Value, Velocity, 0.0f, 0.5f);

	REQUIRE(Value.Equals(FQuat::Identity, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Velocity.Z, 2.0f, 1e-4f));
}

// ---------------------------------------------------------------------------
// DeadBlendExtrapolateScale
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::DeadBlendExtrapolateScale::ScaleExtrapolatesInMultiplicativeSpace", "[SpringMath][unit][MustPass]")
{
	FVector Value(1.0f, 1.0f, 1.0f);
	FVector Velocity(1.0f, 1.0f, 1.0f); // Growing in log-space

	const FVector InitialValue = Value;
	SpringMath::DeadBlendExtrapolateScale(Value, Velocity, 1.0f, 0.5f);

	// Scale should have grown (in multiplicative space, positive velocity means increasing)
	CHECK(Value.X > InitialValue.X);
	CHECK(Value.Y > InitialValue.Y);
	CHECK(Value.Z > InitialValue.Z);
	// Velocity should decay
	REQUIRE(Velocity.Size() < FVector(1.0f, 1.0f, 1.0f).Size());
}

TEST_CASE("SpringMath::DeadBlendExtrapolateScale::TimeZeroNoChange", "[SpringMath][unit][MustPass]")
{
	FVector Value(2.0f, 2.0f, 2.0f);
	FVector Velocity(1.0f, 1.0f, 1.0f);

	SpringMath::DeadBlendExtrapolateScale(Value, Velocity, 0.0f, 0.5f);

	REQUIRE(FMath::IsNearlyEqual(Value.X, 2.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Value.Y, 2.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Value.Z, 2.0f, 1e-4f));
}

// ---------------------------------------------------------------------------
// DeadBlendApply<float>
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::DeadBlendApply::AtTimeZeroOutputIsTransitionValue", "[SpringMath][unit][MustPass]")
{
	float Value = 100.0f;
	float Velocity = 0.0f;
	const float ValueTransition = 50.0f;
	const float VelocityTransition = 10.0f;
	const float BlendTime = 1.0f;
	const float SmoothingTime = 0.5f;

	SpringMath::DeadBlendApply(Value, Velocity, ValueTransition, VelocityTransition, 0.0f, BlendTime, SmoothingTime);

	// At time 0: Alpha = 0, so output = Lerp(Extrapolated, Input, 0) = Extrapolated
	// Extrapolated at time 0 = ValueTransition = 50
	REQUIRE(FMath::IsNearlyEqual(Value, 50.0f, 1e-4f));
}

TEST_CASE("SpringMath::DeadBlendApply::PastBlendTimeOutputIsInput", "[SpringMath][unit][MustPass]")
{
	float Value = 100.0f;
	float Velocity = 0.0f;
	const float ValueTransition = 50.0f;
	const float VelocityTransition = 10.0f;
	const float BlendTime = 1.0f;
	const float SmoothingTime = 0.5f;

	SpringMath::DeadBlendApply(Value, Velocity, ValueTransition, VelocityTransition, 2.0f, BlendTime, SmoothingTime);

	// Past blend time: Alpha = 1, so output = Lerp(Extrapolated, Input, 1) = Input = 100
	REQUIRE(FMath::IsNearlyEqual(Value, 100.0f, 1e-4f));
}

TEST_CASE("SpringMath::DeadBlendApply::MidBlendInterpolates", "[SpringMath][unit][MustPass]")
{
	float Value = 100.0f;
	float Velocity = 0.0f;
	const float ValueTransition = 50.0f;
	const float VelocityTransition = 0.0f;
	const float BlendTime = 1.0f;
	const float SmoothingTime = 0.5f;

	SpringMath::DeadBlendApply(Value, Velocity, ValueTransition, VelocityTransition, 0.5f, BlendTime, SmoothingTime);

	// At half blend: result should be between the extrapolated transition value and the input
	CHECK(Value > 50.0f);
	CHECK(Value < 100.0f);
}

// ---------------------------------------------------------------------------
// DeadBlendTransition<float>
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::DeadBlendTransition::ComputesTransitionState", "[SpringMath][unit][MustPass]")
{
	float ValueTransition = 50.0f;
	float VelocityTransition = 0.0f;
	const float SrcValue = 80.0f;
	const float SrcVelocity = 5.0f;
	const float BlendTime = 1.0f;
	const float SmoothingTime = 0.5f;

	SpringMath::DeadBlendTransition(ValueTransition, VelocityTransition, SrcValue, SrcVelocity, 0.0f, BlendTime, SmoothingTime);

	// At time 0: Alpha=0, so ValueTransition = Lerp(Extrapolated(=50), SrcValue(=80), 0) = 50
	REQUIRE(FMath::IsNearlyEqual(ValueTransition, 50.0f, 1e-4f));
}

TEST_CASE("SpringMath::DeadBlendTransition::PastBlendTimeUsesSource", "[SpringMath][unit][MustPass]")
{
	float ValueTransition = 50.0f;
	float VelocityTransition = 0.0f;
	const float SrcValue = 80.0f;
	const float SrcVelocity = 5.0f;
	const float BlendTime = 1.0f;
	const float SmoothingTime = 0.5f;

	SpringMath::DeadBlendTransition(ValueTransition, VelocityTransition, SrcValue, SrcVelocity, 2.0f, BlendTime, SmoothingTime);

	// Past blend time: Alpha=1, so ValueTransition = Lerp(Extrapolated, Src, 1) = Src
	REQUIRE(FMath::IsNearlyEqual(ValueTransition, 80.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(VelocityTransition, 5.0f, 1e-4f));
}

// ---------------------------------------------------------------------------
// DeadBlendApplyQuat
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::DeadBlendApplyQuat::AtTimeZeroOutputIsTransitionValue", "[SpringMath][unit][MustPass]")
{
	const FQuat TransitionRotation = FQuat(FVector::UpVector, FMath::DegreesToRadians(45.0f));
	const FVector TransitionVelocity = FVector::ZeroVector;

	FQuat Value = FQuat(FVector::UpVector, FMath::DegreesToRadians(90.0f));
	FVector Velocity = FVector::ZeroVector;

	SpringMath::DeadBlendApplyQuat(Value, Velocity, TransitionRotation, TransitionVelocity, 0.0f, 1.0f, 0.5f);

	// At time 0: Alpha=0, so output should be the extrapolated transition value (= TransitionRotation at t=0)
	REQUIRE(Value.Equals(TransitionRotation, 1e-3f));
}

TEST_CASE("SpringMath::DeadBlendApplyQuat::PastBlendTimeOutputIsInput", "[SpringMath][unit][MustPass]")
{
	const FQuat TransitionRotation = FQuat(FVector::UpVector, FMath::DegreesToRadians(45.0f));
	const FVector TransitionVelocity = FVector::ZeroVector;

	const FQuat InputRotation = FQuat(FVector::UpVector, FMath::DegreesToRadians(90.0f));
	FQuat Value = InputRotation;
	FVector Velocity = FVector::ZeroVector;

	SpringMath::DeadBlendApplyQuat(Value, Velocity, TransitionRotation, TransitionVelocity, 2.0f, 1.0f, 0.5f);

	// Past blend time: Alpha=1, output should match the input rotation
	REQUIRE(Value.Equals(InputRotation, 1e-3f));
}

// ---------------------------------------------------------------------------
// DeadBlendTransitionQuat
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::DeadBlendTransitionQuat::ComputesTransitionState", "[SpringMath][unit][MustPass]")
{
	FQuat ValueTransition = FQuat(FVector::UpVector, FMath::DegreesToRadians(30.0f));
	FVector VelocityTransition = FVector::ZeroVector;
	const FQuat SrcValue = FQuat(FVector::UpVector, FMath::DegreesToRadians(90.0f));
	const FVector SrcVelocity = FVector::ZeroVector;

	SpringMath::DeadBlendTransitionQuat(ValueTransition, VelocityTransition, SrcValue, SrcVelocity, 2.0f, 1.0f, 0.5f);

	// Past blend time: Alpha=1, so transition value should become the source
	REQUIRE(ValueTransition.Equals(SrcValue, 1e-3f));
}

// ---------------------------------------------------------------------------
// DeadBlendApplyAngle
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::DeadBlendApplyAngle::AtTimeZeroOutputIsTransitionValue", "[SpringMath][unit][MustPass]")
{
	float Value = FMath::DegreesToRadians(90.0f);
	float Velocity = 0.0f;
	const float ValueTransition = FMath::DegreesToRadians(45.0f);
	const float VelocityTransition = 0.0f;

	SpringMath::DeadBlendApplyAngle(Value, Velocity, ValueTransition, VelocityTransition, 0.0f, 1.0f, 0.5f);

	// At time 0: Alpha=0, result should be the transition value
	REQUIRE(FMath::IsNearlyEqual(Value, FMath::DegreesToRadians(45.0f), 1e-3f));
}

TEST_CASE("SpringMath::DeadBlendApplyAngle::PastBlendTimeOutputIsInput", "[SpringMath][unit][MustPass]")
{
	const float InputAngle = FMath::DegreesToRadians(90.0f);
	float Value = InputAngle;
	float Velocity = 0.0f;
	const float ValueTransition = FMath::DegreesToRadians(45.0f);
	const float VelocityTransition = 0.0f;

	SpringMath::DeadBlendApplyAngle(Value, Velocity, ValueTransition, VelocityTransition, 2.0f, 1.0f, 0.5f);

	// Past blend time: Alpha=1, result should be the original input
	REQUIRE(FMath::IsNearlyEqual(Value, InputAngle, 1e-3f));
}

// ---------------------------------------------------------------------------
// DeadBlendTransitionAngle
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::DeadBlendTransitionAngle::ComputesTransitionState", "[SpringMath][unit][MustPass]")
{
	float ValueTransition = FMath::DegreesToRadians(30.0f);
	float VelocityTransition = 0.0f;
	const float SrcValue = FMath::DegreesToRadians(90.0f);
	const float SrcVelocity = 1.0f;

	SpringMath::DeadBlendTransitionAngle(ValueTransition, VelocityTransition, SrcValue, SrcVelocity, 2.0f, 1.0f, 0.5f);

	// Past blend time: Alpha=1, transition should become the source
	REQUIRE(FMath::IsNearlyEqual(ValueTransition, SrcValue, 1e-3f));
	REQUIRE(FMath::IsNearlyEqual(VelocityTransition, SrcVelocity, 1e-3f));
}

// ---------------------------------------------------------------------------
// DeadBlendApplyScale
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::DeadBlendApplyScale::AtTimeZeroOutputIsTransitionValue", "[SpringMath][unit][MustPass]")
{
	FVector Value(3.0f, 3.0f, 3.0f);
	FVector Velocity(0.0f, 0.0f, 0.0f);
	const FVector ValueTransition(1.5f, 1.5f, 1.5f);
	const FVector VelocityTransition(0.0f, 0.0f, 0.0f);

	SpringMath::DeadBlendApplyScale(Value, Velocity, ValueTransition, VelocityTransition, 0.0f, 1.0f, 0.5f);

	// At time 0: Alpha=0, so ScaleEerp(Extrapolated, Input, 0) = Extrapolated = ValueTransition
	REQUIRE(FMath::IsNearlyEqual(Value.X, 1.5f, 1e-3f));
	REQUIRE(FMath::IsNearlyEqual(Value.Y, 1.5f, 1e-3f));
	REQUIRE(FMath::IsNearlyEqual(Value.Z, 1.5f, 1e-3f));
}

TEST_CASE("SpringMath::DeadBlendApplyScale::PastBlendTimeOutputIsInput", "[SpringMath][unit][MustPass]")
{
	FVector Value(3.0f, 3.0f, 3.0f);
	FVector Velocity(0.0f, 0.0f, 0.0f);
	const FVector ValueTransition(1.5f, 1.5f, 1.5f);
	const FVector VelocityTransition(0.0f, 0.0f, 0.0f);

	SpringMath::DeadBlendApplyScale(Value, Velocity, ValueTransition, VelocityTransition, 2.0f, 1.0f, 0.5f);

	// Past blend time: Alpha=1, ScaleEerp(Extrapolated, Input, 1) = Input
	REQUIRE(FMath::IsNearlyEqual(Value.X, 3.0f, 1e-3f));
	REQUIRE(FMath::IsNearlyEqual(Value.Y, 3.0f, 1e-3f));
	REQUIRE(FMath::IsNearlyEqual(Value.Z, 3.0f, 1e-3f));
}

// ---------------------------------------------------------------------------
// DeadBlendTransitionScale
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::DeadBlendTransitionScale::ComputesTransitionState", "[SpringMath][unit][MustPass]")
{
	FVector ValueTransition(1.0f, 1.0f, 1.0f);
	FVector VelocityTransition(0.0f, 0.0f, 0.0f);
	const FVector SrcValue(2.0f, 2.0f, 2.0f);
	const FVector SrcVelocity(0.0f, 0.0f, 0.0f);

	SpringMath::DeadBlendTransitionScale(ValueTransition, VelocityTransition, SrcValue, SrcVelocity, 2.0f, 1.0f, 0.5f);

	// Past blend time: Alpha=1, ScaleEerp(Extrapolated, Src, 1) = Src
	REQUIRE(FMath::IsNearlyEqual(ValueTransition.X, 2.0f, 1e-3f));
	REQUIRE(FMath::IsNearlyEqual(ValueTransition.Y, 2.0f, 1e-3f));
	REQUIRE(FMath::IsNearlyEqual(ValueTransition.Z, 2.0f, 1e-3f));
}

TEST_CASE("SpringMath::DeadBlendTransitionScale::AtTimeZeroUsesExtrapolated", "[SpringMath][unit][MustPass]")
{
	FVector ValueTransition(1.5f, 1.5f, 1.5f);
	FVector VelocityTransition(0.0f, 0.0f, 0.0f);
	const FVector SrcValue(3.0f, 3.0f, 3.0f);
	const FVector SrcVelocity(0.0f, 0.0f, 0.0f);

	SpringMath::DeadBlendTransitionScale(ValueTransition, VelocityTransition, SrcValue, SrcVelocity, 0.0f, 1.0f, 0.5f);

	// At time 0: Alpha=0, ScaleEerp(Extrapolated(=1.5), Src, 0) = Extrapolated = 1.5
	REQUIRE(FMath::IsNearlyEqual(ValueTransition.X, 1.5f, 1e-3f));
	REQUIRE(FMath::IsNearlyEqual(ValueTransition.Y, 1.5f, 1e-3f));
	REQUIRE(FMath::IsNearlyEqual(ValueTransition.Z, 1.5f, 1e-3f));
}
