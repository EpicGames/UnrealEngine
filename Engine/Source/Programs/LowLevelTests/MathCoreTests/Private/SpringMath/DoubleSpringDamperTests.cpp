// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/catch_test_macros.hpp>

#include "Math/SpringMath.h"

TEST_CASE("SpringMath::CriticalDoubleSpringDamper::FloatConvergesToTarget", "[SpringMath][unit][MustPass]")
{
	// Start at X=10 and converge towards Target=0 over 1000 steps
	float X = 10.0f;
	float V = 0.0f;
	float Xi = 10.0f;
	float Vi = 0.0f;
	const float TargetX = 0.0f;
	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;

	for (int32 Step = 0; Step < 1000; ++Step)
	{
		SpringMath::CriticalDoubleSpringDamper(X, V, Xi, Vi, TargetX, SmoothingTime, DeltaTime);
	}

	REQUIRE(FMath::IsNearlyEqual(X, TargetX, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Xi, TargetX, 1e-4f));
}

TEST_CASE("SpringMath::CriticalDoubleSpringDamper::ZeroSmoothingTimeSnapsToTarget", "[SpringMath][unit][MustPass]")
{
	float X = 50.0f;
	float V = 5.0f;
	float Xi = 50.0f;
	float Vi = 5.0f;
	const float TargetX = -20.0f;
	const float SmoothingTime = 0.0f;
	const float DeltaTime = 1.0f / 60.0f;

	SpringMath::CriticalDoubleSpringDamper(X, V, Xi, Vi, TargetX, SmoothingTime, DeltaTime);

	// SmoothingTime=0 snaps exactly to target
	REQUIRE(X == TargetX);
	REQUIRE(Xi == TargetX);
	REQUIRE(V == 0.0f);
	REQUIRE(Vi == 0.0f);
}

TEST_CASE("SpringMath::CriticalDoubleSpringDamperAngle::AngleWrappingConverges", "[SpringMath][unit][MustPass]")
{
	// Start near +PI, target near -PI. The spring should take the short arc.
	float AngleRadians = 3.0f;
	float AngularVelocityRadians = 0.0f;
	float IntermediateAngleRadians = 3.0f;
	float IntermediateAngularVelocityRadians = 0.0f;
	const float TargetAngleRadians = -3.0f;
	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;

	for (int32 Step = 0; Step < 1000; ++Step)
	{
		SpringMath::CriticalDoubleSpringDamperAngle(
			AngleRadians, AngularVelocityRadians,
			IntermediateAngleRadians, IntermediateAngularVelocityRadians,
			TargetAngleRadians, SmoothingTime, DeltaTime);
	}

	// The result should be near the target angle (modulo 2*PI wrapping)
	const float Delta = FMath::FindDeltaAngleRadians(TargetAngleRadians, AngleRadians);
	REQUIRE(FMath::IsNearlyEqual(Delta, 0.0f, 1e-4f));
}

TEST_CASE("SpringMath::CriticalDoubleSpringDamperQuat::QuatConvergesToTarget", "[SpringMath][unit][MustPass]")
{
	FQuat Rotation = FQuat::Identity;
	FVector AngularVelocityRadians = FVector::ZeroVector;
	FQuat IntermediateRotation = FQuat::Identity;
	FVector IntermediateAngularVelocityRadians = FVector::ZeroVector;

	// Target is a 90-degree rotation around Z
	const FQuat TargetRotation = FQuat(FVector::UpVector, FMath::DegreesToRadians(90.0f));
	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;

	for (int32 Step = 0; Step < 1000; ++Step)
	{
		SpringMath::CriticalDoubleSpringDamperQuat(
			Rotation, AngularVelocityRadians,
			IntermediateRotation, IntermediateAngularVelocityRadians,
			TargetRotation, SmoothingTime, DeltaTime);
	}

	// The rotation should be very close to the target
	const float AngleDiff = Rotation.AngularDistance(TargetRotation);
	REQUIRE(FMath::IsNearlyEqual(AngleDiff, 0.0f, 1e-4f));
}

TEST_CASE("SpringMath::CriticalDoubleSpringDamperScale::ScaleConvergesToTarget", "[SpringMath][unit][MustPass]")
{
	FVector Scale(2.0f, 2.0f, 2.0f);
	FVector ScalarVelocity = FVector::ZeroVector;
	FVector IntermediateScale(2.0f, 2.0f, 2.0f);
	FVector IntermediateScalarVelocity = FVector::ZeroVector;
	const FVector TargetScale(1.0f, 1.0f, 1.0f);
	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;

	for (int32 Step = 0; Step < 1000; ++Step)
	{
		SpringMath::CriticalDoubleSpringDamperScale(
			Scale, ScalarVelocity,
			IntermediateScale, IntermediateScalarVelocity,
			TargetScale, SmoothingTime, DeltaTime);
	}

	REQUIRE(FMath::IsNearlyEqual(Scale.X, TargetScale.X, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Scale.Y, TargetScale.Y, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Scale.Z, TargetScale.Z, 1e-4f));
}
