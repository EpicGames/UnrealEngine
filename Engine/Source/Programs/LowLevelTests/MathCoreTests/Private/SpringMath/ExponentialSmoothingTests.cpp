// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"
#include <catch2/catch_test_macros.hpp>
#include "Math/SpringMath.h"

TEST_CASE("SpringMath::ExponentialSmoothingApproxQuat::ConvergesToTarget", "[SpringMath][unit][MustPass]")
{
	// Start at identity, target at 90-degree rotation about Z
	FQuat Current = FQuat::Identity;
	const FQuat Target = FQuat(FVector::UpVector, FMath::DegreesToRadians(90.0f));

	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;

	// Run many steps to allow convergence
	for (int32 Step = 0; Step < 600; ++Step)
	{
		SpringMath::ExponentialSmoothingApproxQuat(Current, Target, DeltaTime, SmoothingTime);
	}

	// After 10 seconds of simulation at 60fps, the quaternion should have converged to the target
	const float AngleError = Current.AngularDistance(Target);
	REQUIRE(FMath::IsNearlyEqual(AngleError, 0.0f, 1e-4f));
}

TEST_CASE("SpringMath::ExponentialSmoothingApproxQuat::ZeroSmoothingTimeSnapsToTarget", "[SpringMath][unit][MustPass]")
{
	FQuat Current = FQuat::Identity;
	const FQuat Target = FQuat(FVector::UpVector, FMath::DegreesToRadians(90.0f));

	SpringMath::ExponentialSmoothingApproxQuat(Current, Target, 1.0f / 60.0f, 0.0f);

	const float AngleError = Current.AngularDistance(Target);
	REQUIRE(FMath::IsNearlyEqual(AngleError, 0.0f, 1e-4f));
}

TEST_CASE("SpringMath::ExponentialSmoothingApproxAngle::ConvergesToTarget", "[SpringMath][unit][MustPass]")
{
	float CurrentAngle = 0.0f;
	const float TargetAngle = UE_HALF_PI; // PI/2

	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;

	for (int32 Step = 0; Step < 600; ++Step)
	{
		SpringMath::ExponentialSmoothingApproxAngle(CurrentAngle, TargetAngle, DeltaTime, SmoothingTime);
	}

	REQUIRE(FMath::IsNearlyEqual(CurrentAngle, TargetAngle, 1e-4f));
}

TEST_CASE("SpringMath::ExponentialSmoothingApproxAngle::WrappingTakesShortPath", "[SpringMath][unit][MustPass]")
{
	// 350 degrees in radians
	float CurrentAngle = FMath::DegreesToRadians(350.0f);
	// 10 degrees in radians
	const float TargetAngle = FMath::DegreesToRadians(10.0f);

	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;

	// Record initial delta to verify it takes the short path (20 degrees, not 340 degrees)
	const float InitialDelta = FMath::Abs(FMath::FindDeltaAngleRadians(CurrentAngle, TargetAngle));
	REQUIRE(InitialDelta < FMath::DegreesToRadians(25.0f)); // Short path should be ~20 degrees

	for (int32 Step = 0; Step < 600; ++Step)
	{
		SpringMath::ExponentialSmoothingApproxAngle(CurrentAngle, TargetAngle, DeltaTime, SmoothingTime);
	}

	// After convergence, the delta angle should be near zero
	const float FinalDelta = FMath::Abs(FMath::FindDeltaAngleRadians(CurrentAngle, TargetAngle));
	REQUIRE(FMath::IsNearlyEqual(FinalDelta, 0.0f, 1e-4f));
}

TEST_CASE("SpringMath::ExponentialSmoothingApproxAngle::ZeroSmoothingTimeSnapsToTarget", "[SpringMath][unit][MustPass]")
{
	float CurrentAngle = 0.0f;
	const float TargetAngle = UE_HALF_PI;

	SpringMath::ExponentialSmoothingApproxAngle(CurrentAngle, TargetAngle, 1.0f / 60.0f, 0.0f);

	REQUIRE(FMath::IsNearlyEqual(CurrentAngle, TargetAngle, 1e-4f));
}

TEST_CASE("SpringMath::ExponentialSmoothingApproxScale::ConvergesToTarget", "[SpringMath][unit][MustPass]")
{
	FVector CurrentScale(2.0f, 2.0f, 2.0f);
	const FVector TargetScale(1.0f, 1.0f, 1.0f);

	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;

	for (int32 Step = 0; Step < 600; ++Step)
	{
		SpringMath::ExponentialSmoothingApproxScale(CurrentScale, TargetScale, DeltaTime, SmoothingTime);
	}

	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(CurrentScale.X), static_cast<float>(TargetScale.X), 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(CurrentScale.Y), static_cast<float>(TargetScale.Y), 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(CurrentScale.Z), static_cast<float>(TargetScale.Z), 1e-4f));
}

TEST_CASE("SpringMath::ExponentialSmoothingApproxScale::ZeroSmoothingTimeSnapsToTarget", "[SpringMath][unit][MustPass]")
{
	FVector CurrentScale(2.0f, 2.0f, 2.0f);
	const FVector TargetScale(1.0f, 1.0f, 1.0f);

	SpringMath::ExponentialSmoothingApproxScale(CurrentScale, TargetScale, 1.0f / 60.0f, 0.0f);

	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(CurrentScale.X), static_cast<float>(TargetScale.X), 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(CurrentScale.Y), static_cast<float>(TargetScale.Y), 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(CurrentScale.Z), static_cast<float>(TargetScale.Z), 1e-4f));
}
