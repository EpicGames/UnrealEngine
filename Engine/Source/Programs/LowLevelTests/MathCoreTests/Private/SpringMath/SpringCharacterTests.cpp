// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/catch_test_macros.hpp>

#include "Math/SpringMath.h"

TEST_CASE("SpringMath::SpringCharacterUpdate::ConvergesToTargetVelocity", "[SpringMath][unit][MustPass]")
{
	FVector Position = FVector::ZeroVector;
	FVector Velocity = FVector::ZeroVector;
	FVector Acceleration = FVector::ZeroVector;
	const FVector TargetVelocity(100.0f, 0.0f, 0.0f);
	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;

	for (int32 i = 0; i < 1000; ++i)
	{
		SpringMath::SpringCharacterUpdate(Position, Velocity, Acceleration, TargetVelocity, SmoothingTime, DeltaTime);
	}

	REQUIRE(FMath::IsNearlyEqual(Velocity.X, TargetVelocity.X, 1.0f));
	REQUIRE(FMath::IsNearlyEqual(Velocity.Y, TargetVelocity.Y, 1.0f));
	REQUIRE(FMath::IsNearlyEqual(Velocity.Z, TargetVelocity.Z, 1.0f));

	REQUIRE(FMath::IsNearlyEqual(Acceleration.X, 0.0f, 1.0f));
	REQUIRE(FMath::IsNearlyEqual(Acceleration.Y, 0.0f, 1.0f));
	REQUIRE(FMath::IsNearlyEqual(Acceleration.Z, 0.0f, 1.0f));
}

TEST_CASE("SpringMath::SpringCharacterUpdate::DeadzoneSnapsVelocity", "[SpringMath][unit][MustPass]")
{
	FVector Position = FVector::ZeroVector;
	FVector Velocity(99.995f, 0.0f, 0.0f);
	FVector Acceleration = FVector::ZeroVector;
	const FVector TargetVelocity(100.0f, 0.0f, 0.0f);
	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;
	const float VDeadzone = 0.01f;
	const float ADeadzone = 0.0001f;

	// Velocity is within the deadzone of the target, so after one update it should snap
	SpringMath::SpringCharacterUpdate(Position, Velocity, Acceleration, TargetVelocity, SmoothingTime, DeltaTime, VDeadzone, ADeadzone);

	REQUIRE(Velocity.X == TargetVelocity.X);
	REQUIRE(Velocity.Y == TargetVelocity.Y);
	REQUIRE(Velocity.Z == TargetVelocity.Z);
	// Acceleration only snaps to zero if it's within ADeadzone; the spring update may leave residual acceleration
	REQUIRE(FMath::IsNearlyEqual(Acceleration.X, 0.0f, 0.01f));
	REQUIRE(FMath::IsNearlyEqual(Acceleration.Y, 0.0f, 0.01f));
	REQUIRE(FMath::IsNearlyEqual(Acceleration.Z, 0.0f, 0.01f));
}

TEST_CASE("SpringMath::SpringCharacterPredict::MatchesDirectUpdate", "[SpringMath][unit][MustPass]")
{
	const FVector InitPosition = FVector::ZeroVector;
	const FVector InitVelocity = FVector::ZeroVector;
	const FVector InitAcceleration = FVector::ZeroVector;
	const FVector TargetVelocity(100.0f, 0.0f, 0.0f);
	const float SmoothingTime = 0.5f;
	const float SecondsPerStep = 1.0f / 60.0f;
	constexpr int32 PredictCount = 10;

	TArray<FVector> PredPositions;
	TArray<FVector> PredVelocities;
	TArray<FVector> PredAccelerations;
	PredPositions.SetNum(PredictCount);
	PredVelocities.SetNum(PredictCount);
	PredAccelerations.SetNum(PredictCount);

	SpringMath::SpringCharacterPredict<FVector>(
		PredPositions, PredVelocities, PredAccelerations,
		InitPosition, InitVelocity, InitAcceleration,
		TargetVelocity, SmoothingTime, SecondsPerStep);

	CHECK(PredPositions.Num() == PredictCount);
	CHECK(PredVelocities.Num() == PredictCount);
	CHECK(PredAccelerations.Num() == PredictCount);

	// Verify each prediction step matches calling SpringCharacterUpdate directly with the
	// corresponding elapsed time. SpringCharacterPredict copies initial state then calls
	// SpringCharacterUpdate with PredictTime = (i+1) * SecondsPerStep, so a single-step
	// update from the initial state with that time should yield the same result.
	for (int32 i = 0; i < PredictCount; ++i)
	{
		FVector DirectPos = InitPosition;
		FVector DirectVel = InitVelocity;
		FVector DirectAcc = InitAcceleration;
		const float PredictTime = static_cast<float>(i + 1) * SecondsPerStep;

		SpringMath::SpringCharacterUpdate(DirectPos, DirectVel, DirectAcc, TargetVelocity, SmoothingTime, PredictTime);

		REQUIRE(FMath::IsNearlyEqual(PredPositions[i].X, DirectPos.X, 1e-3f));
		REQUIRE(FMath::IsNearlyEqual(PredPositions[i].Y, DirectPos.Y, 1e-3f));
		REQUIRE(FMath::IsNearlyEqual(PredPositions[i].Z, DirectPos.Z, 1e-3f));

		REQUIRE(FMath::IsNearlyEqual(PredVelocities[i].X, DirectVel.X, 1e-3f));
		REQUIRE(FMath::IsNearlyEqual(PredVelocities[i].Y, DirectVel.Y, 1e-3f));
		REQUIRE(FMath::IsNearlyEqual(PredVelocities[i].Z, DirectVel.Z, 1e-3f));

		REQUIRE(FMath::IsNearlyEqual(PredAccelerations[i].X, DirectAcc.X, 1e-3f));
		REQUIRE(FMath::IsNearlyEqual(PredAccelerations[i].Y, DirectAcc.Y, 1e-3f));
		REQUIRE(FMath::IsNearlyEqual(PredAccelerations[i].Z, DirectAcc.Z, 1e-3f));
	}
}

TEST_CASE("SpringMath::VelocitySpringCharacterUpdate::ConvergesToTargetVelocity", "[SpringMath][unit][MustPass]")
{
	FVector Position = FVector::ZeroVector;
	FVector Velocity = FVector::ZeroVector;
	FVector VelocityIntermediate = FVector::ZeroVector;
	FVector Acceleration = FVector::ZeroVector;
	const FVector TargetVelocity(100.0f, 0.0f, 0.0f);
	const float SmoothingTime = 0.5f;
	const float MaxAcceleration = 500.0f;
	const float DeltaTime = 1.0f / 60.0f;

	for (int32 i = 0; i < 1000; ++i)
	{
		SpringMath::VelocitySpringCharacterUpdate(
			Position, Velocity, VelocityIntermediate, Acceleration,
			TargetVelocity, SmoothingTime, MaxAcceleration, DeltaTime);
	}

	REQUIRE(FMath::IsNearlyEqual(Velocity.X, TargetVelocity.X, 1.0f));
	REQUIRE(FMath::IsNearlyEqual(Velocity.Y, TargetVelocity.Y, 1.0f));
	REQUIRE(FMath::IsNearlyEqual(Velocity.Z, TargetVelocity.Z, 1.0f));
}

TEST_CASE("SpringMath::VelocitySpringCharacterPredict::OutputCountsMatch", "[SpringMath][unit][MustPass]")
{
	const FVector InitPosition = FVector::ZeroVector;
	const FVector InitVelocity = FVector::ZeroVector;
	const FVector InitIntermediateVelocity = FVector::ZeroVector;
	const FVector InitAcceleration = FVector::ZeroVector;
	const FVector TargetVelocity(100.0f, 0.0f, 0.0f);
	const float SmoothingTime = 0.5f;
	const float MaxAcceleration = 500.0f;
	const float SecondsPerStep = 1.0f / 60.0f;
	constexpr int32 PredictCount = 10;

	TArray<FVector> PredPositions;
	TArray<FVector> PredVelocities;
	TArray<FVector> PredIntermediateVelocities;
	TArray<FVector> PredAccelerations;
	PredPositions.SetNum(PredictCount);
	PredVelocities.SetNum(PredictCount);
	PredIntermediateVelocities.SetNum(PredictCount);
	PredAccelerations.SetNum(PredictCount);

	SpringMath::VelocitySpringCharacterPredict<FVector>(
		PredPositions, PredVelocities, PredIntermediateVelocities, PredAccelerations,
		InitPosition, InitVelocity, InitIntermediateVelocity, InitAcceleration,
		TargetVelocity, SmoothingTime, MaxAcceleration, SecondsPerStep);

	REQUIRE(PredPositions.Num() == PredictCount);
	REQUIRE(PredVelocities.Num() == PredictCount);
	REQUIRE(PredIntermediateVelocities.Num() == PredictCount);
	REQUIRE(PredAccelerations.Num() == PredictCount);

	// Verify predictions match direct single-step updates from initial state
	for (int32 i = 0; i < PredictCount; ++i)
	{
		FVector DirectPos = InitPosition;
		FVector DirectVel = InitVelocity;
		FVector DirectIntVel = InitIntermediateVelocity;
		FVector DirectAcc = InitAcceleration;
		const float PredictTime = static_cast<float>(i + 1) * SecondsPerStep;

		SpringMath::VelocitySpringCharacterUpdate(
			DirectPos, DirectVel, DirectIntVel, DirectAcc,
			TargetVelocity, SmoothingTime, MaxAcceleration, PredictTime);

		REQUIRE(FMath::IsNearlyEqual(PredPositions[i].X, DirectPos.X, 1e-3f));
		REQUIRE(FMath::IsNearlyEqual(PredVelocities[i].X, DirectVel.X, 1e-3f));
	}
}

TEST_CASE("SpringMath::CriticalSpringDamperQuatPredict::OutputCountMatches", "[SpringMath][unit][MustPass]")
{
	const FQuat CurrentRotation = FQuat::Identity;
	const FVector CurrentAngularVelocity = FVector::ZeroVector;
	const FQuat TargetRotation(FRotator(0.0f, 90.0f, 0.0f).Quaternion());
	const float SmoothingTime = 0.5f;
	const float SecondsPerStep = 1.0f / 60.0f;
	constexpr int32 PredictCount = 10;

	TArray<FQuat> PredRotations;
	TArray<FVector> PredAngularVelocities;
	PredRotations.SetNum(PredictCount);
	PredAngularVelocities.SetNum(PredictCount);

	SpringMath::CriticalSpringDamperQuatPredict(
		PredRotations, PredAngularVelocities,
		PredictCount, CurrentRotation, CurrentAngularVelocity,
		TargetRotation, SmoothingTime, SecondsPerStep);

	REQUIRE(PredRotations.Num() == PredictCount);
	REQUIRE(PredAngularVelocities.Num() == PredictCount);

	// The first predicted rotation should have moved from identity towards the target
	const float DotWithIdentity = FMath::Abs(PredRotations[0].IsNormalized()
		? PredRotations[0] | FQuat::Identity
		: 0.0f);

	CHECK(DotWithIdentity < 1.0f);
}
