// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/catch_test_macros.hpp>

#include "Math/UnrealMathUtility.h"

inline bool ApproxEquals(double A, double B, double Threshold)
{
	return FMath::Abs(A - B) < Threshold;
}

inline double AngularFrequencyFromHz(double FrequencyHz)
{
	return FrequencyHz * TWO_PI;
}

inline double CriticalSmoothingTimeFromHz(double FrequencyHz)
{
	// Here W = UndampedFrequency * TWO_PI, and SmoothingTime = 2/W
	return 2.0 / AngularFrequencyFromHz(FrequencyHz);
}

inline void IntegrateSpringDamper(
	double& Value, double& ValueRate, double Target, double TargetRate, double DeltaTime,
	double UndampedFrequencyHz, double  DampingRatio, int NumSteps)
{
	for (int Step = 0; Step < NumSteps; ++Step)
	{
		FMath::SpringDamper(
			Value, ValueRate, Target, TargetRate, DeltaTime, UndampedFrequencyHz, DampingRatio);
	}
}

inline void IntegrateCriticallyDampedSmoothing(
	double& Value, double& ValueRate, double  Target, double  TargetRate, double  DeltaTime,
	double SmoothingTime, int NumSteps)
{
	for (int Step = 0; Step < NumSteps; ++Step)
	{
		FMath::CriticallyDampedSmoothing(Value, ValueRate, Target, TargetRate, DeltaTime, SmoothingTime);
	}
}

// Calculates the exact solution for target position and velocity of zero
inline void CalculateExpectedCriticalDampedState(
	double& OutX, double& OutV, double x0, double v0, double w0, double t)
{
	const double e = FMath::Exp(-w0 * t);
	OutX = (x0 + (v0 + w0 * x0) * t) * e;
	OutV = (v0 - w0 * (v0 + w0 * x0) * t) * e;
}

// Calculates the exact solution for arbitrary target position and velocity
inline void CalculateExpectedCriticalDampedState(
	double& OutX, double& OutV, double x0, double v0, double xt, double vt, double w0, double t)
{
	const double SmoothingTime = 2.0 / w0;
	const double xa = xt + vt * SmoothingTime; // adjusted target

	const double dx0 = x0 - xa;

	const double e = FMath::Exp(-w0 * t);

	// Analytic solution
	OutX = xa + (dx0 + (v0 + w0 * dx0) * t) * e;
	OutV = (v0 - w0 * (v0 + w0 * dx0) * t) * e;
}

inline void RequireApprox(
	double Value, double ValueRate, double ExpectedValue, double ExpectedValueRate, double RelativeTol = 0.05)
{
	// Tolerance is a fraction of the result - but don't let it be zero!
	const double ValueTol = FMath::Max(FMath::Abs(ExpectedValue) * RelativeTol, KINDA_SMALL_NUMBER);
	const double ValueRateTol = FMath::Max(FMath::Abs(ExpectedValueRate) * RelativeTol, KINDA_SMALL_NUMBER);

	REQUIRE(ApproxEquals(Value, ExpectedValue, ValueTol));
	REQUIRE(ApproxEquals(ValueRate, ExpectedValueRate, ValueRateTol));
}

TEST_CASE("UnrealMathUtility::SpringDamper", "[Core][Math][unit][MustPass]")
{
	// Frequency in Hz, which we call strength
	double UndampedFrequencyHz = 1.0;

	// Undamped - let it go round N.5 times and check that it has negated
	{
		const double Target = 0.0;
		const double TargetRate = 0.0;

		const double DampingRatio = 0.0;
		const float Time = 3.5 / UndampedFrequencyHz;

		const double DeltaTime = 0.01;

		double Value = 1.0;
		double ValueRate = 0.0;

		IntegrateSpringDamper(
			Value, ValueRate, Target, TargetRate, DeltaTime, UndampedFrequencyHz, DampingRatio, Time / DeltaTime);

		RequireApprox(Value, ValueRate, -1.0, 0.0, 0.01);
	}

	// highly damped
	{
		const double Target = 0.0;
		const double TargetRate = 0.0;

		const double DampingRatio = 1.0;
		const float Time = 10.5 / UndampedFrequencyHz;

		const double DeltaTime = 0.01;

		double Value = 1.0;
		double ValueRate = 1.0;

		IntegrateSpringDamper(
			Value, ValueRate, Target, TargetRate, DeltaTime, UndampedFrequencyHz, DampingRatio, Time / DeltaTime);

		RequireApprox(Value, ValueRate, 0.0, 0.0, 0.01);
	}

	// critically damped - compare with the exact solution
	{
		const double Target = 0.0;
		const double TargetRate = 0.0;

		const double DeltaTime = 0.01;
		const double DampingRatio = 1.0;

		const float Time = 0.5 / UndampedFrequencyHz;

		double X0 = 1.0;
		double V0 = 1.0;

		double Value = X0;
		double ValueRate = V0;

		IntegrateSpringDamper(
			Value, ValueRate, Target, TargetRate, DeltaTime, UndampedFrequencyHz, DampingRatio, Time / DeltaTime);

		const double W0 = AngularFrequencyFromHz(UndampedFrequencyHz);

		double X, V;
		CalculateExpectedCriticalDampedState(X, V, X0, V0, Target, TargetRate, W0, Time);

		// Approximate calculations are accumulated over multiple steps - 5% is OK
		RequireApprox(Value, ValueRate, X, V, 0.05);
	}
}

TEST_CASE("UnrealMathUtility::CriticallyDampedSmoothing", "[Core][Math][unit][MustPass]")
{
	// Frequency in Hz, which we call strength
	const double UndampedFrequencyHz = 1.0;

	const double Target = 0.0;
	const double TargetRate = 0.0;

	const double DampingRatio = 1.0;

	const float Time = 0.5 / UndampedFrequencyHz;
	double DeltaTime = 0.01;

	const double X0 = 1.0;
	const double V0 = 1.0;

	double W0 = AngularFrequencyFromHz(UndampedFrequencyHz);
	const double SmoothingTime = CriticalSmoothingTimeFromHz(UndampedFrequencyHz);

	double Value = X0;
	double ValueRate = V0;

	IntegrateCriticallyDampedSmoothing(
		Value, ValueRate, Target, TargetRate, DeltaTime, SmoothingTime, Time / DeltaTime);

	double X, V;
	CalculateExpectedCriticalDampedState(X, V, X0, V0, Target, TargetRate, W0, Time);

	RequireApprox(Value, ValueRate, X, V, 0.05);
}
