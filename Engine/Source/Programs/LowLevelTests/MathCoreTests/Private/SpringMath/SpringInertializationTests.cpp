// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"
#include <catch2/catch_test_macros.hpp>
#include "Math/SpringMath.h"

// ---------------------------------------------------------------------------
// SpringInertializeApply<float>
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::SpringInertializeApply::FullOffsetAtTimeZero", "[SpringMath][unit][MustPass]")
{
	// At time 0, CriticalDecay has not decayed anything, so the full offset should be applied
	float Value = 0.0f;
	float Velocity = 0.0f;
	const float ValueOffset = 5.0f;
	const float VelocityOffset = 0.0f;

	SpringMath::SpringInertializeApply(Value, Velocity, ValueOffset, VelocityOffset, 0.0f, 0.5f);

	REQUIRE(FMath::IsNearlyEqual(Value, 5.0f, 1e-4f));
}

TEST_CASE("SpringMath::SpringInertializeApply::OffsetDecaysToZeroOverTime", "[SpringMath][unit][MustPass]")
{
	// After a large time, the offset should have decayed to nearly zero
	float Value = 0.0f;
	float Velocity = 0.0f;
	const float ValueOffset = 5.0f;
	const float VelocityOffset = 0.0f;

	SpringMath::SpringInertializeApply(Value, Velocity, ValueOffset, VelocityOffset, 10.0f, 0.5f);

	// After 10 seconds with SmoothingTime=0.5, the offset should have decayed significantly from 5.0
	REQUIRE(FMath::Abs(Value) < 0.5f);
}

// ---------------------------------------------------------------------------
// SpringInertializeTransition<float>
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::SpringInertializeTransition::ComputesOffsetFromSrcDstDifference", "[SpringMath][unit][MustPass]")
{
	// Src=10, Dst=5 with no prior offset and TimeSinceTransition=0
	float ValueOffset = 0.0f;
	float VelocityOffset = 0.0f;

	const float SrcValue = 10.0f;
	const float SrcVelocity = 0.0f;
	const float DstValue = 5.0f;
	const float DstVelocity = 0.0f;

	SpringMath::SpringInertializeTransition(
		ValueOffset, VelocityOffset,
		SrcValue, SrcVelocity,
		DstValue, DstVelocity,
		0.0f, 0.5f);

	// At t=0, CriticalDecay does not change the existing offset (which is 0),
	// so ValueOffset = SrcValue + 0 - DstValue = 5
	REQUIRE(FMath::IsNearlyEqual(ValueOffset, 5.0f, 1e-4f));
}

// ---------------------------------------------------------------------------
// SpringInertializeApplyQuat
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::SpringInertializeApplyQuat::RotationOffsetDecaysToIdentity", "[SpringMath][unit][MustPass]")
{
	// Start with a 45-degree rotation offset about Z
	FQuat Value = FQuat::Identity;
	FVector AngularVelocity = FVector::ZeroVector;
	const FQuat RotationOffset = FQuat(FVector::UpVector, FMath::DegreesToRadians(45.0f));
	const FVector AngularVelocityOffset = FVector::ZeroVector;

	// At t=0, full offset should be applied
	FQuat ValueAtZero = Value;
	FVector VelAtZero = AngularVelocity;
	SpringMath::SpringInertializeApplyQuat(ValueAtZero, VelAtZero, RotationOffset, AngularVelocityOffset, 0.0f, 0.5f);

	float AngleAtZero = ValueAtZero.AngularDistance(FQuat::Identity);
	CHECK(AngleAtZero > FMath::DegreesToRadians(40.0f));

	// After a large time, the offset should have decayed and rotation should be near identity
	FQuat ValueLater = FQuat::Identity;
	FVector VelLater = FVector::ZeroVector;
	SpringMath::SpringInertializeApplyQuat(ValueLater, VelLater, RotationOffset, AngularVelocityOffset, 10.0f, 0.5f);

	float AngleLater = ValueLater.AngularDistance(FQuat::Identity);
	REQUIRE(FMath::IsNearlyEqual(AngleLater, 0.0f, 0.01f));
}

// ---------------------------------------------------------------------------
// SpringInertializeTransitionQuat
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::SpringInertializeTransitionQuat::ComputesRotationOffset", "[SpringMath][unit][MustPass]")
{
	FQuat RotationOffset = FQuat::Identity;
	FVector AngularVelocityOffset = FVector::ZeroVector;

	const FQuat SrcRotation = FQuat(FVector::UpVector, FMath::DegreesToRadians(90.0f));
	const FVector SrcAngularVelocity = FVector::ZeroVector;
	const FQuat DstRotation = FQuat::Identity;
	const FVector DstAngularVelocity = FVector::ZeroVector;

	SpringMath::SpringInertializeTransitionQuat(
		RotationOffset, AngularVelocityOffset,
		SrcRotation, SrcAngularVelocity,
		DstRotation, DstAngularVelocity,
		0.0f, 0.5f);

	// The offset should capture the difference: SrcRotation * Identity * DstRotation.Inverse = SrcRotation
	float OffsetAngle = RotationOffset.AngularDistance(SrcRotation);
	REQUIRE(FMath::IsNearlyEqual(OffsetAngle, 0.0f, 1e-4f));
}

// ---------------------------------------------------------------------------
// SpringInertializeApplyAngle
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::SpringInertializeApplyAngle::AngleOffsetDecaysToZero", "[SpringMath][unit][MustPass]")
{
	// At t=0, full offset applied
	float AngleAtZero = 0.0f;
	float VelAtZero = 0.0f;
	SpringMath::SpringInertializeApplyAngle(AngleAtZero, VelAtZero, 1.0f, 0.0f, 0.0f, 0.5f);
	REQUIRE(FMath::IsNearlyEqual(AngleAtZero, 1.0f, 1e-4f));

	// After a large time, offset decays to zero
	float AngleLater = 0.0f;
	float VelLater = 0.0f;
	SpringMath::SpringInertializeApplyAngle(AngleLater, VelLater, 1.0f, 0.0f, 10.0f, 0.5f);
	REQUIRE(FMath::IsNearlyEqual(AngleLater, 0.0f, 0.01f));
}

// ---------------------------------------------------------------------------
// SpringInertializeTransitionAngle
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::SpringInertializeTransitionAngle::ComputesAngleOffset", "[SpringMath][unit][MustPass]")
{
	float AngleOffset = 0.0f;
	float AngularVelocityOffset = 0.0f;

	const float SrcAngle = FMath::DegreesToRadians(90.0f);
	const float SrcAngularVelocity = 0.0f;
	const float DstAngle = FMath::DegreesToRadians(45.0f);
	const float DstAngularVelocity = 0.0f;

	SpringMath::SpringInertializeTransitionAngle(
		AngleOffset, AngularVelocityOffset,
		SrcAngle, SrcAngularVelocity,
		DstAngle, DstAngularVelocity,
		0.0f, 0.5f);

	// Offset should be the short-path delta from Dst to Src: ~45 degrees
	const float ExpectedOffset = FMath::FindDeltaAngleRadians(DstAngle, SrcAngle);
	REQUIRE(FMath::IsNearlyEqual(AngleOffset, ExpectedOffset, 1e-4f));
}

// ---------------------------------------------------------------------------
// SpringInertializeApplyScale
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::SpringInertializeApplyScale::ScaleOffsetDecaysToIdentity", "[SpringMath][unit][MustPass]")
{
	// Scale offset of (2,2,2) means initial scale is doubled
	const FVector ScaleOffset(2.0f, 2.0f, 2.0f);
	const FVector VelocityOffset = FVector::ZeroVector;

	// At t=0, the full scale offset is applied multiplicatively
	FVector ScaleAtZero(1.0f, 1.0f, 1.0f);
	FVector VelAtZero = FVector::ZeroVector;
	SpringMath::SpringInertializeApplyScale(ScaleAtZero, VelAtZero, ScaleOffset, VelocityOffset, 0.0f, 0.5f);

	CHECK(FMath::IsNearlyEqual(static_cast<float>(ScaleAtZero.X), 2.0f, 1e-4f));

	// After a large time, the offset decays to (1,1,1) identity scale
	FVector ScaleLater(1.0f, 1.0f, 1.0f);
	FVector VelLater = FVector::ZeroVector;
	SpringMath::SpringInertializeApplyScale(ScaleLater, VelLater, ScaleOffset, VelocityOffset, 10.0f, 0.5f);

	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(ScaleLater.X), 1.0f, 0.01f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(ScaleLater.Y), 1.0f, 0.01f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(ScaleLater.Z), 1.0f, 0.01f));
}

// ---------------------------------------------------------------------------
// SpringInertializeTransitionScale
// ---------------------------------------------------------------------------

TEST_CASE("SpringMath::SpringInertializeTransitionScale::ComputesScaleOffset", "[SpringMath][unit][MustPass]")
{
	FVector ScaleOffset(1.0f, 1.0f, 1.0f); // Identity offset
	FVector VelocityOffset = FVector::ZeroVector;

	const FVector SrcScale(2.0f, 2.0f, 2.0f);
	const FVector SrcVelocity = FVector::ZeroVector;
	const FVector DstScale(1.0f, 1.0f, 1.0f);
	const FVector DstVelocity = FVector::ZeroVector;

	SpringMath::SpringInertializeTransitionScale(
		ScaleOffset, VelocityOffset,
		SrcScale, SrcVelocity,
		DstScale, DstVelocity,
		0.0f, 0.5f);

	// Offset = SrcScale * decayed(ScaleOffset) / DstScale
	// At t=0, decayed identity offset stays (1,1,1), so offset = (2*1)/1 = 2
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(ScaleOffset.X), 2.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(ScaleOffset.Y), 2.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(ScaleOffset.Z), 2.0f, 1e-4f));
}
