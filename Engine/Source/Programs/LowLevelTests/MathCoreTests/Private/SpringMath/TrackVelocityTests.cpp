// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"
#include <catch2/catch_test_macros.hpp>
#include "Math/SpringMath.h"

TEST_CASE("SpringMath::TrackVelocity::ConstantVelocitySignal", "[SpringMath][unit][MustPass]")
{
	// Track a constant-velocity signal: value moves by 10 per second
	const float Speed = 10.0f;
	const float DeltaTime = 1.0f / 60.0f;

	float TrackedValue = 0.0f;
	float TrackedVelocity = 0.0f;

	// First call establishes the baseline
	float CurrentValue = 0.0f;
	SpringMath::TrackVelocity(TrackedValue, TrackedVelocity, CurrentValue, DeltaTime);

	// Second call with value advanced by Speed * DeltaTime
	CurrentValue = Speed * DeltaTime;
	SpringMath::TrackVelocity(TrackedValue, TrackedVelocity, CurrentValue, DeltaTime);

	REQUIRE(FMath::IsNearlyEqual(TrackedVelocity, Speed, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(TrackedValue, CurrentValue, 1e-4f));
}

TEST_CASE("SpringMath::TrackVelocity::ZeroDeltaTimeProducesZeroVelocity", "[SpringMath][unit][MustPass]")
{
	float TrackedValue = 5.0f;
	float TrackedVelocity = 100.0f; // Intentionally non-zero

	const float NewValue = 10.0f;
	SpringMath::TrackVelocity(TrackedValue, TrackedVelocity, NewValue, 0.0f);

	REQUIRE(FMath::IsNearlyEqual(TrackedVelocity, 0.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(TrackedValue, NewValue, 1e-4f));
}

TEST_CASE("SpringMath::TrackVelocityQuat::ConstantAngularVelocity", "[SpringMath][unit][MustPass]")
{
	// Rotate by a known amount per frame about the Z axis
	const float AngularSpeedRadPerSec = UE_PI; // 180 deg/s
	const float DeltaTime = 1.0f / 60.0f;
	const float AnglePerFrame = AngularSpeedRadPerSec * DeltaTime;

	FQuat TrackedValue = FQuat::Identity;
	FVector TrackedVelocity = FVector::ZeroVector;

	// First call: establish baseline
	FQuat CurrentValue = FQuat::Identity;
	SpringMath::TrackVelocityQuat(TrackedValue, TrackedVelocity, CurrentValue, DeltaTime);

	// Second call: rotate by one frame's worth about Z
	CurrentValue = FQuat(FVector::UpVector, AnglePerFrame);
	SpringMath::TrackVelocityQuat(TrackedValue, TrackedVelocity, CurrentValue, DeltaTime);

	// The velocity should be approximately (0, 0, AngularSpeedRadPerSec) since we rotate about Z
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(TrackedVelocity.X), 0.0f, 1e-2f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(TrackedVelocity.Y), 0.0f, 1e-2f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(TrackedVelocity.Z), AngularSpeedRadPerSec, 1e-2f));
}

TEST_CASE("SpringMath::TrackVelocityQuat::ZeroDeltaTimeProducesZeroVelocity", "[SpringMath][unit][MustPass]")
{
	FQuat TrackedValue = FQuat::Identity;
	FVector TrackedVelocity(1.0f, 1.0f, 1.0f);

	const FQuat NewValue = FQuat(FVector::UpVector, FMath::DegreesToRadians(45.0f));
	SpringMath::TrackVelocityQuat(TrackedValue, TrackedVelocity, NewValue, 0.0f);

	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(TrackedVelocity.X), 0.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(TrackedVelocity.Y), 0.0f, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(TrackedVelocity.Z), 0.0f, 1e-4f));
}

TEST_CASE("SpringMath::TrackVelocityAngle::ConstantAngularChange", "[SpringMath][unit][MustPass]")
{
	const float AngularSpeed = 2.0f; // rad/s
	const float DeltaTime = 1.0f / 60.0f;

	float TrackedValue = 0.0f;
	float TrackedVelocity = 0.0f;

	// Establish baseline
	float CurrentAngle = 0.0f;
	SpringMath::TrackVelocityAngle(TrackedValue, TrackedVelocity, CurrentAngle, DeltaTime);

	// Advance by one frame
	CurrentAngle = AngularSpeed * DeltaTime;
	SpringMath::TrackVelocityAngle(TrackedValue, TrackedVelocity, CurrentAngle, DeltaTime);

	REQUIRE(FMath::IsNearlyEqual(TrackedVelocity, AngularSpeed, 1e-4f));
}

TEST_CASE("SpringMath::TrackVelocityAngle::ZeroDeltaTimeProducesZeroVelocity", "[SpringMath][unit][MustPass]")
{
	float TrackedValue = 0.5f;
	float TrackedVelocity = 99.0f;

	SpringMath::TrackVelocityAngle(TrackedValue, TrackedVelocity, 1.0f, 0.0f);

	REQUIRE(FMath::IsNearlyEqual(TrackedVelocity, 0.0f, 1e-4f));
}

TEST_CASE("SpringMath::TrackVelocityScale::ConstantScaleChange", "[SpringMath][unit][MustPass]")
{
	// Scale changes from 1 to 2 over 1 second, so the log-space velocity should be ln(2)/dt per second
	const float DeltaTime = 1.0f / 60.0f;

	// Starting scale
	FVector TrackedValue(1.0f, 1.0f, 1.0f);
	FVector TrackedVelocity = FVector::ZeroVector;

	// Establish baseline
	FVector CurrentScale(1.0f, 1.0f, 1.0f);
	SpringMath::TrackVelocityScale(TrackedValue, TrackedVelocity, CurrentScale, DeltaTime);

	// Scale grows exponentially: multiply by e^(rate * dt)
	// Use a constant rate of 1.0 in log space per second
	const float LogRate = 1.0f;
	CurrentScale = FVector(
		FMath::Exp(LogRate * DeltaTime),
		FMath::Exp(LogRate * DeltaTime),
		FMath::Exp(LogRate * DeltaTime));
	SpringMath::TrackVelocityScale(TrackedValue, TrackedVelocity, CurrentScale, DeltaTime);

	// The log-space velocity should be approximately LogRate
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(TrackedVelocity.X), LogRate, 1e-2f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(TrackedVelocity.Y), LogRate, 1e-2f));
	REQUIRE(FMath::IsNearlyEqual(static_cast<float>(TrackedVelocity.Z), LogRate, 1e-2f));
}
