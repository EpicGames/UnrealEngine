// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "TestHarness.h"

#include <catch2/catch_test_macros.hpp>

#include "Math/SpringMath.h"

TEST_CASE("SpringMath::VelocitySpringDamperF::FloatConvergesToTarget", "[SpringMath][unit][MustPass]")
{
	float X = 0.0f;
	float V = 0.0f;
	float Xi = 0.0f;
	const float TargetX = 100.0f;
	const float MaxSpeed = 10.0f;
	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;

	// Run for enough steps to converge (target is 100 units away at max speed 10 => ~10s minimum)
	const int32 NumSteps = 1200;
	for (int32 Step = 0; Step < NumSteps; ++Step)
	{
		SpringMath::VelocitySpringDamperF(X, V, Xi, TargetX, MaxSpeed, SmoothingTime, DeltaTime);
	}

	REQUIRE(FMath::IsNearlyEqual(X, TargetX, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Xi, TargetX, 1e-4f));
}

TEST_CASE("SpringMath::VelocitySpringDamperF::IntermediateTargetMovesAtControlledSpeed", "[SpringMath][unit][MustPass]")
{
	float X = 0.0f;
	float V = 0.0f;
	float Xi = 0.0f;
	const float TargetX = 100.0f;
	const float MaxSpeed = 10.0f;
	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;

	float PrevXi = Xi;

	// Run a moderate number of steps while Xi has not yet reached the target
	for (int32 Step = 0; Step < 300; ++Step)
	{
		SpringMath::VelocitySpringDamperF(X, V, Xi, TargetX, MaxSpeed, SmoothingTime, DeltaTime);

		// While Xi has not reached the target, verify its speed is bounded by MaxSpeed
		if (!FMath::IsNearlyEqual(Xi, TargetX, 1e-4f))
		{
			const float XiSpeed = FMath::Abs(Xi - PrevXi) / DeltaTime;
			CHECK(XiSpeed <= MaxSpeed + 1e-4f);
		}

		PrevXi = Xi;
	}
}

TEST_CASE("SpringMath::VelocitySpringDamper::VectorConvergesToTarget", "[SpringMath][unit][MustPass]")
{
	FVector X = FVector::ZeroVector;
	FVector V = FVector::ZeroVector;
	FVector Xi = FVector::ZeroVector;
	const FVector TargetX(100.0f, 0.0f, 0.0f);
	const float MaxSpeed = 10.0f;
	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;

	// Run for enough steps to converge
	const int32 NumSteps = 1200;
	for (int32 Step = 0; Step < NumSteps; ++Step)
	{
		SpringMath::VelocitySpringDamper(X, V, Xi, TargetX, MaxSpeed, SmoothingTime, DeltaTime);
	}

	REQUIRE(FMath::IsNearlyEqual(X.X, TargetX.X, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(X.Y, TargetX.Y, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(X.Z, TargetX.Z, 1e-4f));

	REQUIRE(FMath::IsNearlyEqual(Xi.X, TargetX.X, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Xi.Y, TargetX.Y, 1e-4f));
	REQUIRE(FMath::IsNearlyEqual(Xi.Z, TargetX.Z, 1e-4f));
}

TEST_CASE("SpringMath::VelocitySpringDamper::IntermediateTargetSpeedIsBoundedByMaxSpeed", "[SpringMath][unit][MustPass]")
{
	FVector X = FVector::ZeroVector;
	FVector V = FVector::ZeroVector;
	FVector Xi = FVector::ZeroVector;
	const FVector TargetX(100.0f, 0.0f, 0.0f);
	const float MaxSpeed = 10.0f;
	const float SmoothingTime = 0.5f;
	const float DeltaTime = 1.0f / 60.0f;

	FVector PrevXi = Xi;

	// Run a moderate number of steps while Xi has not yet reached the target
	for (int32 Step = 0; Step < 300; ++Step)
	{
		SpringMath::VelocitySpringDamper(X, V, Xi, TargetX, MaxSpeed, SmoothingTime, DeltaTime);

		// While Xi has not reached the target, verify its speed is bounded by MaxSpeed
		if (!Xi.Equals(TargetX, 1e-4f))
		{
			const float XiDisplacement = (Xi - PrevXi).Length();
			const float XiSpeed = XiDisplacement / DeltaTime;
			CHECK(XiSpeed <= MaxSpeed + 1e-4f);
		}

		PrevXi = Xi;
	}
}
