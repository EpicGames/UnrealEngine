// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TransformStateStream.h"
#include "MockStateStream.generated.h"

// This is the interface Gameplay is using to interact with statestream.

////////////////////////////////////////////////////////////////////////////////////////////////////
// Static state. Should be immutable during the life time of the instance. Should use attributes to generate serialization code etc using UHT.
USTRUCT(StateStreamStaticState)
struct FMockStaticState
{
	GENERATED_USTRUCT_BODY()
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Dynamic state. Can change over the lifetime of the instance.
USTRUCT(StateStreamDynamicState)
struct FMockDynamicState
{
	GENERATED_USTRUCT_BODY()
	FMockDynamicState(uint32 V) { SetValue(V); }
	FMockDynamicState(uint32 V, bool B) { SetValue(V); SetBit2(B); }

private:

	UPROPERTY()
	uint32 Value = 0;

	UPROPERTY()
	FTransformHandle Transform;

	UPROPERTY()
	uint32 bBit1 : 1;

	UPROPERTY()
	uint32 bBit2 : 1;
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Handle to keep track of instances and their lifetime. Is ref counted so when count reaches 0 instance is tagged as deleted
USTRUCT(StateStreamHandle)
struct FMockHandle : public FStateStreamHandle
{
	GENERATED_USTRUCT_BODY()
};


////////////////////////////////////////////////////////////////////////////////////////////////////
// Id. Used to identify statestreams. For finding statestream game side and register dependencies render side.
inline constexpr uint32 MockStateStreamId = 128;


////////////////////////////////////////////////////////////////////////////////////////////////////
// The statestream itself that Gameplay is using to create instances.
class IMockStateStream
{
public:
	DECLARE_STATESTREAM(Mock)
	virtual Handle Game_CreateInstance(const FMockStaticState& Ss, const FMockDynamicState& Ds, uint32 LaneId = DefaultLaneId) = 0;
};
