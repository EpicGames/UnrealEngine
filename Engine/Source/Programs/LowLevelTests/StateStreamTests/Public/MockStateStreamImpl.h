// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericStateStream.h"
#include "MockStateStream.h"
#include "TransformStateStreamImpl.h"

// ============ This is ONLY RT side ==============

////////////////////////////////////////////////////////////////////////////////////////////////////

struct MockObject : public FTransformObjectListener
{
	virtual ~MockObject()
	{
		SetTransformObject({});
	}

	void SetTransformObject(const FTransformHandle& Handle)
	{
		if (Transform)
		{
			Transform->RemoveListener(this);
		}
		Transform = (FTransformObject*)Handle.Render_GetUserData();
		if (Transform)
		{
			Transform->AddListener(this);
		}
	}

	virtual void OnTransformObjectDirty() override
	{
		// Do stuff
	}

	float Value = 0;
	bool Bit = false;
	TRefCountPtr<FTransformObject> Transform;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FMockStateStreamImpl : public TStateStream<TStateStreamSettings<IMockStateStream, MockObject>>
{
public:
	TArray<MockObject*> Instances;
	uint32 CreateCount = 0;
	uint32 CreateAndDestroyCount = 0;
	uint32 UpdateCount = 0;
	uint32 DestroyCount = 0;

	virtual void Render_OnCreate(const FMockStaticState& Ss, const FMockDynamicState& Ds, MockObject*& UserData, bool IsDestroyedInSameFrame) override
	{
		if (IsDestroyedInSameFrame)
		{
			++CreateAndDestroyCount;
			return;
		}

		MockObject* Object = new MockObject(); 
		Object->Value = Ds.GetValue();
		Object->Bit = Ds.GetBit2();
		UserData = Object;

		if (Ds.GetTransform().IsValid())
		{
			Object->SetTransformObject(Ds.GetTransform());
		}

		Instances.Add(Object);
		++CreateCount;
	}

	virtual void Render_OnUpdate(const FMockStaticState& Ss, const FMockDynamicState& Ds, MockObject*& UserData) override
	{
		MockObject* Object = UserData;
		if (Ds.ValueModified())
		{
			Object->Value = Ds.GetValue();
		}
		if (Ds.Bit2Modified())
		{
			Object->Bit = Ds.GetBit2();
		}
		if (Ds.TransformModified())
		{
			Object->SetTransformObject(Ds.GetTransform());
		}
		++UpdateCount;
	}

	virtual void Render_OnDestroy(const FMockStaticState& Ss, const FMockDynamicState& Ds, MockObject*& UserData) override
	{
		MockObject* Object = UserData;
		if (!Object)
		{
			return;
		}
		Instances.Remove(Object);
		delete Object;
		UserData = nullptr;
		++DestroyCount;
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////
