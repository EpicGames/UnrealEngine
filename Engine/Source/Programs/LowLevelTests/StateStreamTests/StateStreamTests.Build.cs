// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

public class StateStreamTests : TestModuleRules
{
	public StateStreamTests(ReadOnlyTargetRules Target) : base(Target, true)
	{
		TestMetadata = new Metadata()
		{
			TestName = "StateStream",
			TestShortName = "StateStream",
			ReportType = "xml",
			SupportedPlatforms = {
				UnrealTargetPlatform.Win64,
				UnrealTargetPlatform.Linux,
				UnrealTargetPlatform.Mac,
				UnrealTargetPlatform.Android,
				UnrealTargetPlatform.IOS }
		};

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"StateStream",
			});
	}
}
