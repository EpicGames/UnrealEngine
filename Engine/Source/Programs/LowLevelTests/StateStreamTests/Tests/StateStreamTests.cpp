// Copyright Epic Games, Inc. All Rights Reserved.
#include "TestCommon/Initialization.h"
#include "StateStreamCreator.h"
#include "TestHarness.h"
#include "MockStateStreamImpl.h"
#include "StateStreamManagerImpl.h"
#include "TransformStateStreamImpl.h"


TEST_CASE("FPow2ChunkedArray", "[StateStream]")
{
	//   0 -  15 Bucket 0  (16 entries)
	//  16 -  47 Bucket 1  (32 entries)
	//  48 - 111 Bucket 2  (64 entries)
	// 112 - 239 Bucket 3  (128 entries)
	// 240 - 495 Bucket 4  (256 entries)

	FPow2ChunkedArray<int> Store;
	CHECK_EQUAL(Store.SkipCount, 4);
	CHECK_EQUAL(Store.BucketCount, 21);
	CHECK_EQUAL(Store.GetBucketIndex(0), 0);
	CHECK_EQUAL(Store.GetBucketIndex(15), 0);
	CHECK_EQUAL(Store.GetBucketIndex(16), 1);
	CHECK_EQUAL(Store.GetBucketIndex(47), 1);
	CHECK_EQUAL(Store.GetBucketIndex(48), 2);
	CHECK_EQUAL(Store.GetBucketIndex(111), 2);
	CHECK_EQUAL(Store.GetBucketIndex(112), 3);
	CHECK_EQUAL(Store.GetBucketIndex(16777215), Store.BucketCount - 1);

	CHECK_EQUAL(Store.GetBucketSize(0), 16);
	CHECK_EQUAL(Store.GetBucketSize(1), 32);
	CHECK_EQUAL(Store.GetBucketSize(2), 64);
	CHECK_EQUAL(Store.GetBucketSize(3), 128);

	CHECK_EQUAL(Store.GetBucketStart(0), 0);
	CHECK_EQUAL(Store.GetBucketStart(1), 16);
	CHECK_EQUAL(Store.GetBucketStart(2), 48);
	CHECK_EQUAL(Store.GetBucketStart(3), 112);
	CHECK_EQUAL(Store.GetBucketStart(4), 240);

	for (int i=0;i!=1000;++i)
	{
		Store.Add(i);
	}

	for (int i=0;i!=1000;++i)
	{
		CHECK_EQUAL(Store[i], i);
	}
}

TEST_CASE("StateStream::Store", "[StateStream]")
{
	TStateStreamStore<int> Store;

	for (int i=0;i!=1000;++i)
	{
		Store.Add(i);
	}
	CHECK_EQUAL(Store.GetUsedCount(), 1000);

	for (int i=0;i!=1000;++i)
	{
		CHECK_EQUAL(Store[i], i);
	}

	Store.Remove(100);
	Store.Remove(101);

	CHECK_EQUAL(Store.GetUsedCount(), 998);
	CHECK_EQUAL(Store.Add(1234), 101);
	CHECK_EQUAL(Store.Add(1235), 100);
	CHECK_EQUAL(Store.GetUsedCount(), 1000);
	CHECK_EQUAL(Store[100], 1235);
	CHECK_EQUAL(Store[101], 1234);
	CHECK_EQUAL(Store.Add(1235), 1000);
	CHECK_EQUAL(Store.GetUsedCount(), 1001);
	CHECK_EQUAL(Store[1000], 1235);
}

TEST_CASE("StateStream::SingleInstance", "[StateStream]")
{
	// Initialization (needs to be done centrally/renderside before gameplay starts) (These instances are not visible to GT)
	FMockStateStreamImpl MockStateStreamImpl;
	FStateStreamManagerImpl ManagerImpl;
	ManagerImpl.Render_Register(MockStateStreamImpl, false);
	uint32 LaneId = ManagerImpl.Game_CreateLane();
	ManagerImpl.Game_SetDefaultLane(LaneId);

	// Game thread populating ticks
	{
		IStateStreamManager& Manager = ManagerImpl;
		IMockStateStream& Stream = Manager.Game_Get<IMockStateStream>();

		// Tick 1, create an instance
		Manager.Game_BeginTick(LaneId);
		FMockHandle Handle = Stream.Game_CreateInstance(FMockStaticState(), FMockDynamicState{0});
		Manager.Game_EndTick(100, LaneId);

		// Tick 2, update the instance. Set Value to 100
		Manager.Game_BeginTick(LaneId);
		Handle.Update(FMockDynamicState{100, true});
		Handle.Update(FMockDynamicState{100});
		Manager.Game_EndTick(200, LaneId);

		// Tick 3, update the instance again. Set Value to 200
		Manager.Game_BeginTick(LaneId);
		Handle.Update(FMockDynamicState{200, false});
		Manager.Game_EndTick(300, LaneId);

		// Tick 4, destroy the instance
		Manager.Game_BeginTick(LaneId);
		Handle = {};
		Manager.Game_EndTick(400, LaneId);

		// Tick 5, empty
		Manager.Game_BeginTick(LaneId);
		Manager.Game_EndTick(500, LaneId);
	}


	// Render thread consuming ticks

	FStateStreamManagerImpl& Manager = ManagerImpl;
	FMockStateStreamImpl& Stream = MockStateStreamImpl;


	SECTION("Rend not crossing ticks")
    {
		Manager.Render_Update(10); // We start consuming Tick 1
		CHECK_EQUAL(Stream.Instances.Num(), 1);
		CHECK_EQUAL(Stream.Instances[0]->Value, 0);
		CHECK_EQUAL(Stream.Instances[0]->Bit, false);

		Manager.Render_Update(50); // Update from Tick 1 (still no interpolation, instance was created in this tick)
		CHECK_EQUAL(Stream.Instances.Num(), 1);
		CHECK_EQUAL(Stream.Instances[0]->Value, 0);
		CHECK_EQUAL(Stream.Instances[0]->Bit, false);

		Manager.Render_Update(100); // Values should be exactly like 1
		CHECK_EQUAL(Stream.Instances.Num(), 1);
		CHECK_EQUAL(Stream.Instances[0]->Value, 0);
		CHECK_EQUAL(Stream.Instances[0]->Bit, false);

		CHECK_EQUAL(Stream.CreateCount, 1);
		CHECK_EQUAL(Stream.CreateAndDestroyCount, 0);
		CHECK_EQUAL(Stream.DestroyCount, 0);
		CHECK_EQUAL(Stream.UpdateCount, 0);

		// Moving into Tick 2

		Manager.Render_Update(110); // 10 units into Tick 2.. Values should start interpolating from Tick 1 against Tick 2.
		CHECK_EQUAL(Stream.Instances.Num(), 1);
		CHECK_EQUAL(Stream.Instances[0]->Value, 10);
		CHECK_EQUAL(Stream.Instances[0]->Bit, true);

		Manager.Render_Update(150); // 50 units into Tick 2
		CHECK_EQUAL(Stream.Instances.Num(), 1);
		CHECK_EQUAL(Stream.Instances[0]->Value, 50);
		CHECK_EQUAL(Stream.Instances[0]->Bit, true);

		Manager.Render_Update(200); // Full Tick 2.. Value should be like Tick 2
		CHECK_EQUAL(Stream.Instances.Num(), 1);
		CHECK_EQUAL(Stream.Instances[0]->Value, 100);
		CHECK_EQUAL(Stream.Instances[0]->Bit, true);

		CHECK_EQUAL(Stream.CreateCount, 1);
		CHECK_EQUAL(Stream.CreateAndDestroyCount, 0);
		CHECK_EQUAL(Stream.DestroyCount, 0);
		CHECK_EQUAL(Stream.UpdateCount, 3);

		// Moving into Tick 3

		Manager.Render_Update(210); // 10 units into Tick 3.. 
		CHECK_EQUAL(Stream.Instances.Num(), 1);
		CHECK_EQUAL(Stream.Instances[0]->Value, 110);
		CHECK_EQUAL(Stream.Instances[0]->Bit, false);

		Manager.Render_Update(280); // 80 units into Tick 3.. 
		CHECK_EQUAL(Stream.Instances.Num(), 1);
		CHECK_EQUAL(Stream.Instances[0]->Value, 180);

		Manager.Render_Update(300); // Full Tick 3.. 
		CHECK_EQUAL(Stream.Instances.Num(), 1);
		CHECK_EQUAL(Stream.Instances[0]->Value, 200);

		CHECK_EQUAL(Stream.CreateCount, 1);
		CHECK_EQUAL(Stream.CreateAndDestroyCount, 0);
		CHECK_EQUAL(Stream.DestroyCount, 0);
		CHECK_EQUAL(Stream.UpdateCount, 6);

		// Moving into Tick 4

		Manager.Render_Update(310); // 10 units into Tick 4
		CHECK_EQUAL(Stream.Instances.Num(), 0);

		CHECK_EQUAL(Stream.CreateCount, 1);
		CHECK_EQUAL(Stream.CreateAndDestroyCount, 0);
		CHECK_EQUAL(Stream.DestroyCount, 1);
		CHECK_EQUAL(Stream.UpdateCount, 6);
    }


	SECTION("Rend crossing single ticks")
    {
		// Consuming entire Tick 1 and moving into Tick 2

		Manager.Render_Update(110);
		CHECK_EQUAL(Stream.Instances.Num(), 1);
		CHECK_EQUAL(Stream.Instances[0]->Value, 10);

		CHECK_EQUAL(Stream.CreateCount, 1);
		CHECK_EQUAL(Stream.CreateAndDestroyCount, 0);
		CHECK_EQUAL(Stream.DestroyCount, 0);
		CHECK_EQUAL(Stream.UpdateCount, 0);

		// Consuming rest of Tick 2 and moving into Tick 3

		Manager.Render_Update(210);
		CHECK_EQUAL(Stream.Instances.Num(), 1);
		CHECK_EQUAL(Stream.Instances[0]->Value, 110);

		CHECK_EQUAL(Stream.CreateCount, 1);
		CHECK_EQUAL(Stream.CreateAndDestroyCount, 0);
		CHECK_EQUAL(Stream.DestroyCount, 0);
		CHECK_EQUAL(Stream.UpdateCount, 1);

		// Consuming rest of Tick 3 and moving into Tick 4

		Manager.Render_Update(310);
		CHECK_EQUAL(Stream.Instances.Num(), 0);

		CHECK_EQUAL(Stream.CreateCount, 1);
		CHECK_EQUAL(Stream.CreateAndDestroyCount, 0);
		CHECK_EQUAL(Stream.DestroyCount, 1);
		CHECK_EQUAL(Stream.UpdateCount, 1);
	}


	SECTION("Rend crossing multiple ticks")
    {
		// Consuming entire Tick 1 and 2 and moving into Tick 3

		Manager.Render_Update(210);
		CHECK_EQUAL(Stream.Instances.Num(), 1);
		CHECK_EQUAL(Stream.Instances[0]->Value, 110);

		CHECK_EQUAL(Stream.CreateCount, 1);
		CHECK_EQUAL(Stream.CreateAndDestroyCount, 0);
		CHECK_EQUAL(Stream.DestroyCount, 0);
		CHECK_EQUAL(Stream.UpdateCount, 0);

		// Consuming rest of Tick 3, entire 4 and entering 5

		Manager.Render_Update(410);
		CHECK_EQUAL(Stream.Instances.Num(), 0);

		CHECK_EQUAL(Stream.CreateCount, 1);
		CHECK_EQUAL(Stream.CreateAndDestroyCount, 0);
		CHECK_EQUAL(Stream.DestroyCount, 1);
		CHECK_EQUAL(Stream.UpdateCount, 0);
	}


	SECTION("Rend crossing all ticks")
    {
		// Consuming entire Tick 1 to 5

		Manager.Render_Update(500);
		CHECK_EQUAL(Stream.Instances.Num(), 0);
		CHECK_EQUAL(Stream.CreateCount, 0);
		CHECK_EQUAL(Stream.CreateAndDestroyCount, 1);
		CHECK_EQUAL(Stream.DestroyCount, 0);
		CHECK_EQUAL(Stream.UpdateCount, 0);
	}
}

TEST_CASE("StateStream::MultipleInstances", "[StateStream]")
{
	FMockStateStreamImpl MockStateStreamImpl;
	FStateStreamManagerImpl ManagerImpl;
	ManagerImpl.Render_Register(MockStateStreamImpl, false);
	uint32 LaneId = ManagerImpl.Game_CreateLane();
	ManagerImpl.Game_SetDefaultLane(LaneId);

	{
		IStateStreamManager& Manager = ManagerImpl;
		IMockStateStream& Stream = Manager.Game_Get<IMockStateStream>();

		// Tick 1
		Manager.Game_BeginTick(LaneId);
		FMockHandle Handle1 = Stream.Game_CreateInstance(FMockStaticState(), FMockDynamicState{0});
		FMockHandle Handle2 = Stream.Game_CreateInstance(FMockStaticState(), FMockDynamicState{0});
		Manager.Game_EndTick(100, LaneId);

		// Tick 2
		Manager.Game_BeginTick(LaneId);
		Handle1 = {};
		Manager.Game_EndTick(200, LaneId);

		// Tick 3
		Manager.Game_BeginTick(LaneId);
		Handle2 = {};
		Manager.Game_EndTick(300, LaneId);

		// Tick 4
		Manager.Game_BeginTick(LaneId);
		Manager.Game_EndTick(400, LaneId);
	}

	FStateStreamManagerImpl& Manager = ManagerImpl;
	FMockStateStreamImpl& Stream = MockStateStreamImpl;

	SECTION("Rend not crossing ticks")
    {
		Manager.Render_Update(10); // We start consuming Tick 1
		CHECK_EQUAL(Stream.Instances.Num(), 2);

		Manager.Render_Update(100); // Consume rest of Tick 1
		CHECK_EQUAL(Stream.Instances.Num(), 2);

		Manager.Render_Update(110); // Moving into Tick 2
		CHECK_EQUAL(Stream.Instances.Num(), 1);

		Manager.Render_Update(200); // Consume rest of Tick 2
		CHECK_EQUAL(Stream.Instances.Num(), 1);

		Manager.Render_Update(210); // Moving into Tick 3
		CHECK_EQUAL(Stream.Instances.Num(), 0);

		Manager.Render_Update(300); // Consume rest of Tick 3
		CHECK_EQUAL(Stream.Instances.Num(), 0);

		Manager.Render_Update(400); // Consume rest of Tick 4
		CHECK_EQUAL(Stream.Instances.Num(), 0);
	}
}

TEST_CASE("StateStream::GarbageCollect", "[StateStream]")
{
	FMockStateStreamImpl MockStateStreamImpl;
	FStateStreamManagerImpl ManagerImpl;
	ManagerImpl.Render_Register(MockStateStreamImpl, false);
	uint32 LaneId = ManagerImpl.Game_CreateLane();
	ManagerImpl.Game_SetDefaultLane(LaneId);

	{
		IStateStreamManager& Manager = ManagerImpl;
		IMockStateStream& Stream = Manager.Game_Get<IMockStateStream>();

		// Tick 1
		Manager.Game_BeginTick(LaneId);
		FMockHandle Handle1 = Stream.Game_CreateInstance(FMockStaticState(), FMockDynamicState{0});
		FMockHandle Handle2 = Stream.Game_CreateInstance(FMockStaticState(), FMockDynamicState{0});
		Manager.Game_EndTick(100, LaneId);
		CHECK_EQUAL(MockStateStreamImpl.GetUsedInstancesCount(), 2);
		CHECK_EQUAL(MockStateStreamImpl.GetUsedDynamicStatesCount(), 2);

		// Tick 2
		Manager.Game_BeginTick(LaneId);
		Handle1 = {};
		Handle2 = {};
		Manager.Game_EndTick(200, LaneId);

		// Tick 3
		Manager.Game_BeginTick(LaneId);
		Manager.Game_EndTick(300, LaneId);
		CHECK_EQUAL(MockStateStreamImpl.GetUsedInstancesCount(), 2);
		CHECK_EQUAL(MockStateStreamImpl.GetUsedDynamicStatesCount(), 2);
	}

	FStateStreamManagerImpl& Manager = ManagerImpl;
	FMockStateStreamImpl& Stream = MockStateStreamImpl;
	{
		Manager.Render_Update(10);
		Manager.Render_GarbageCollect();
		CHECK_EQUAL(MockStateStreamImpl.GetUsedDynamicStatesCount(), 2);

		Manager.Render_Update(100);
		Manager.Render_GarbageCollect();
		CHECK_EQUAL(MockStateStreamImpl.GetUsedDynamicStatesCount(), 2);

		Manager.Render_Update(110);
		Manager.Render_GarbageCollect();
		CHECK_EQUAL(MockStateStreamImpl.GetUsedDynamicStatesCount(), 2);

		Manager.Render_Update(200);
		Manager.Render_GarbageCollect();

		Manager.Render_Update(210);
		Manager.Render_GarbageCollect();
		CHECK_EQUAL(Stream.GetUsedInstancesCount(), 2);
		CHECK_EQUAL(MockStateStreamImpl.GetUsedDynamicStatesCount(), 2);

		Manager.Render_Update(300);
		Manager.Render_GarbageCollect();
		CHECK_EQUAL(Stream.GetUsedInstancesCount(), 0);
		CHECK_EQUAL(MockStateStreamImpl.GetUsedDynamicStatesCount(), 0);
	}
}

TEST_CASE("StateStream::InternalDependencies", "[StateStream]")
{
	FTransformStateStreamImpl TransformStreamImpl;

	FStateStreamManagerImpl ManagerImpl;
	ManagerImpl.Render_Register(TransformStreamImpl, false);
	uint32 LaneId = ManagerImpl.Game_CreateLane();
	ManagerImpl.Game_SetDefaultLane(LaneId);

	{
		IStateStreamManager& Manager = ManagerImpl;
		ITransformStateStream& TransformStream = Manager.Game_Get<ITransformStateStream>();

		FTransformHandle TransformHandle;

		// Tick 1
		Manager.Game_BeginTick(LaneId);
		{
			FTransformDynamicState Ds;
			Ds.SetParent(TransformStream.Game_CreateInstance(FTransformStaticState(), FTransformDynamicState{}));
			TransformHandle = TransformStream.Game_CreateInstance(FTransformStaticState(), Ds);
		}
		Manager.Game_EndTick(100, LaneId);
		CHECK_EQUAL(TransformStreamImpl.GetUsedInstancesCount(), 2);
		CHECK_EQUAL(TransformStreamImpl.GetUsedDynamicStatesCount(), 2);

		// Tick 2
		Manager.Game_BeginTick(LaneId);
		{
			FTransformDynamicState Ds;
			Ds.SetParent(TransformStream.Game_CreateInstance(FTransformStaticState(), FTransformDynamicState{}));
			TransformHandle.Update(Ds);
		}
		Manager.Game_EndTick(200, LaneId);

		// Tick 3
		Manager.Game_BeginTick(LaneId);
		TransformHandle = {};
		Manager.Game_EndTick(300, LaneId);

		Manager.Game_Exit();
	}


	FStateStreamManagerImpl& Manager = ManagerImpl;
	FTransformStateStreamImpl& TransformStream = TransformStreamImpl;

	SECTION("Rend not crossing ticks")
    {
		// Frame 1 consuming tick 1
		Manager.Render_Update(100);

		// Frame 2 consuming tick 2
		Manager.Render_Update(200);

		// Frame 3 consuming tick 3
		Manager.Render_Update(300);
	}

	SECTION("Rend crossing ticks")
    {
		// Frame 1 consuming tick 1, 2 and 3
		Manager.Render_Update(300);
	}
}

TEST_CASE("StateStream::ExternalDependencies", "[StateStream]")
{
	FTransformStateStreamImpl TransformStreamImpl;
	FMockStateStreamImpl MockStreamImpl;

	FStateStreamManagerImpl ManagerImpl;
	ManagerImpl.Render_Register(TransformStreamImpl, false);
	ManagerImpl.Render_Register(MockStreamImpl, false);

	uint32 LaneId = ManagerImpl.Game_CreateLane();
	ManagerImpl.Game_SetDefaultLane(LaneId);


	{
		IStateStreamManager& Manager = ManagerImpl;
		ITransformStateStream& TransformStream = Manager.Game_Get<ITransformStateStream>();
		IMockStateStream& MockStream = Manager.Game_Get<IMockStateStream>();

		FMockHandle MockHandle;

		// Tick 1
		Manager.Game_BeginTick(LaneId);
		{
			FMockDynamicState Ds;
			Ds.SetTransform(TransformStream.Game_CreateInstance(FTransformStaticState(), FTransformDynamicState{}));
			MockHandle = MockStream.Game_CreateInstance(FMockStaticState(), Ds);
		}
		Manager.Game_EndTick(100, LaneId);
		CHECK_EQUAL(TransformStreamImpl.GetUsedInstancesCount(), 1);
		CHECK_EQUAL(TransformStreamImpl.GetUsedDynamicStatesCount(), 1);
		CHECK_EQUAL(MockStreamImpl.GetUsedInstancesCount(), 1);
		CHECK_EQUAL(MockStreamImpl.GetUsedDynamicStatesCount(), 1);

		// Tick 2
		Manager.Game_BeginTick(LaneId);
		{
			FMockDynamicState Ds;
			Ds.SetTransform(TransformStream.Game_CreateInstance(FTransformStaticState(), FTransformDynamicState{}));
			MockHandle.Update(Ds);
		}
		Manager.Game_EndTick(200, LaneId);

		// Tick 3
		Manager.Game_BeginTick(LaneId);
		MockHandle = {};
		Manager.Game_EndTick(300, LaneId);

		Manager.Game_Exit();
	}


	FStateStreamManagerImpl& Manager = ManagerImpl;
	FTransformStateStreamImpl& TransformStream = TransformStreamImpl;
	FMockStateStreamImpl& MockStream = MockStreamImpl;

	SECTION("Rend not crossing ticks")
    {
		// Frame 1 consuming tick 1
		Manager.Render_Update(100);
		CHECK_EQUAL(MockStream.Instances.Num(), 1);
		//CHECK_EQUAL(MockStream.Instances[0]->Value, 110);

		// Frame 2 consuming tick 2
		Manager.Render_Update(200);

		// Frame 3 consuming tick 3
		Manager.Render_Update(300);
	}

	SECTION("Rend crossing ticks")
    {
		// Frame 1 consuming tick 1, 2 and 3
		Manager.Render_Update(300);
	}
}

TEST_CASE("StateStream::Interpolation", "[StateStream]")
{
	FTransformStateStreamImpl TransformStreamImpl;
	FStateStreamManagerImpl ManagerImpl;
	ManagerImpl.Render_Register(TransformStreamImpl, false);

	uint32 LaneId = ManagerImpl.Game_CreateLane();
	ManagerImpl.Game_SetDefaultLane(LaneId);

	{
		IStateStreamManager& Manager = ManagerImpl;
		ITransformStateStream& Stream = Manager.Game_Get<ITransformStateStream>();

		// Tick 1
		Manager.Game_BeginTick(LaneId);
		FTransformHandle Handle = Stream.Game_CreateInstance(FTransformStaticState(), FTransformDynamicState());
		Manager.Game_EndTick(0, LaneId);

		// Tick 2
		Manager.Game_BeginTick(LaneId);
		FTransformDynamicState newState;
		newState.SetLocalTransform(FTransform(FQuat::Identity, FVector(100, 100, 100)));
		Handle.Update(newState);
		Manager.Game_EndTick(100, LaneId);

		// Tick 3
		Manager.Game_BeginTick(LaneId);
		Handle = {};
		Manager.Game_EndTick(200, LaneId);
	}

	FStateStreamManagerImpl& Manager = ManagerImpl;
	FTransformStateStreamImpl& Stream = TransformStreamImpl;

	SECTION("Interpolation of translation")
    {
		Manager.Render_Update(10); // We start consuming Tick 2
		FTransformObject* Object = (FTransformObject*)Stream.Render_GetUserData(1u);
		CHECK_EQUAL((FMath::Abs(Object->GetInfo().WorldTransform.GetTranslation().X - 10.0f) < 0.001f), true);
	}
}

TEST_CASE("StateStreamCreator", "[StateStream]")
{
	uint32 Counter = 0;

	FStateStreamCreator Creator1(1, [&](const FStateStreamRegisterContext&){ CHECK_EQUAL(Counter++, 1); }, [&](const FStateStreamUnregisterContext&){ --Counter; });
	FStateStreamCreator Creator2(3, [&](const FStateStreamRegisterContext&){ CHECK_EQUAL(Counter++, 3); }, [&](const FStateStreamUnregisterContext&){ --Counter; });
	FStateStreamCreator Creator3(2, [&](const FStateStreamRegisterContext&){ CHECK_EQUAL(Counter++, 2); }, [&](const FStateStreamUnregisterContext&){ --Counter; });
	FStateStreamCreator Creator4(0, [&](const FStateStreamRegisterContext&){ CHECK_EQUAL(Counter++, 0); }, [&](const FStateStreamUnregisterContext&){ --Counter; });
	FStateStreamCreator Creator5(4, [&](const FStateStreamRegisterContext&){ CHECK_EQUAL(Counter++, 4); }, [&](const FStateStreamUnregisterContext&){ --Counter; });

	FStateStreamManagerImpl ManagerImpl;
	FStateStreamCreator::RegisterStateStreams(FStateStreamRegisterContext{ManagerImpl});
	CHECK_EQUAL(Counter, 5);
	FStateStreamCreator::UnregisterStateStreams(FStateStreamUnregisterContext{ManagerImpl});
	CHECK_EQUAL(Counter, 0);
}
