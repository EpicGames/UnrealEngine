<?xml version="1.0" encoding="utf-8"?>
<RunSettings>
    <!-- Adapter Specific sections -->
    <Catch2Adapter disabled="false">
        <FilenameFilter>Test</FilenameFilter>
		<DiscoverTimeout>60000</DiscoverTimeout>
		<DiscoverCommandLine>--list-tests --verbosity high</DiscoverCommandLine>
		<Logging>Verbose</Logging>
		<ForceListContent>true</ForceListContent>
    </Catch2Adapter>
</RunSettings>